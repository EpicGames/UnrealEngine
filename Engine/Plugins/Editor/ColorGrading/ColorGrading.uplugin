{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Color Grading",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"Description": "Adds a panel with detailed color grading controls",
	"Category": "Misc",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Hidden": true,
	"Modules": [
		{
			"Name": "ColorGradingEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ObjectMixer",
			"Enabled": true
		}
	]
}
