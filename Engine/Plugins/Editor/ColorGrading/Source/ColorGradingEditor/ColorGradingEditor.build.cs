// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class ColorGradingEditor : ModuleRules
{
	public ColorGradingEditor(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AdvancedWidgets",
				"SceneOutliner",
				"ObjectMixerEditor"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"AppFramework",
				"Core",
				"CoreUObject",
				"CinematicCamera",
				"DetailCustomizations",
				"EditorStyle",
				"Engine",
				"InputCore",
				"LevelEditor",
				"PropertyEditor",
				"PropertyPath",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"ToolWidgets",
				"UnrealEd",
				"WorkspaceMenuStructure",
			});
	}
}
