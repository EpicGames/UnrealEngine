// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorGradingCommands.h"

#define LOCTEXT_NAMESPACE "ColorGradingEditor"

void FColorGradingCommands::RegisterCommands()
{
	UI_COMMAND(SaturationColorWheelVisibility, "Saturation", "Show or hide the saturation color wheel", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ContrastColorWheelVisibility, "Constrast", "Show or hide the constrast color wheel", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ColorWheelSliderOrientationHorizontal, "Right", "Puts the color wheel sliders to the right of the color wheel", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ColorWheelSliderOrientationVertical, "Below", "Puts the color wheel sliders below the color wheel", EUserInterfaceActionType::RadioButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE
