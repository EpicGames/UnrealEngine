// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ColorGradingEditorStyle.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

/** Command list for the color grading drawer */
class FColorGradingCommands
	: public TCommands<FColorGradingCommands>
{
public:
	FColorGradingCommands()
		: TCommands<FColorGradingCommands>(TEXT("ColorGrading"),
			NSLOCTEXT("Contexts", "ColorGrading", "Display Cluster Color Grading"), NAME_None, FColorGradingEditorStyle::Get().GetStyleSetName())
	{ }

	virtual void RegisterCommands() override;

public:
	TSharedPtr<FUICommandInfo> SaturationColorWheelVisibility;
	TSharedPtr<FUICommandInfo> ContrastColorWheelVisibility;
	TSharedPtr<FUICommandInfo> ColorWheelSliderOrientationHorizontal;
	TSharedPtr<FUICommandInfo> ColorWheelSliderOrientationVertical;
};