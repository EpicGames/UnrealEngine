// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorGradingEditorDataModel.h"

#include "ColorGradingPanelState.h"

#include "DetailWidgetRow.h"
#include "IPropertyRowGenerator.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"


#define LOCTEXT_NAMESPACE "ColorGradingEditor"

TMap<TWeakObjectPtr<UClass>, FGetDetailsDataModelGenerator> FColorGradingEditorDataModel::RegisteredDataModelGenerators;

/** Detail customizer intended for color FVector4 properties that don't generate property nodes for the child components of the vector, to speed up property node tree generation */
class FFastColorStructCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FFastColorStructCustomization);
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
		HeaderRow.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
	}
};

class FColorPropertyTypeIdentifier : public IPropertyTypeIdentifier
{
	virtual bool IsPropertyTypeCustomized(const IPropertyHandle& PropertyHandle) const override
	{
		return PropertyHandle.HasMetaData(TEXT("ColorGradingMode"));
	}
};

FColorGradingEditorDataModel::FColorGradingEditorDataModel()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FPropertyRowGeneratorArgs Args;
	PropertyRowGenerator = PropertyEditorModule.CreatePropertyRowGenerator(Args);
	PropertyRowGenerator->OnRowsRefreshed().AddRaw(this, &FColorGradingEditorDataModel::OnPropertyRowGeneratorRefreshed);

	TSharedRef<FColorPropertyTypeIdentifier> ColorPropertyTypeIdentifier = MakeShared<FColorPropertyTypeIdentifier>();

	// Since there is an entirely custom set of widgets for displaying and editing the color grading settings, set a customizer for any color vectors to prevent the 
	// property row generator from generating child properties or extraneous widgets, which drastically helps improve performance when loading object properties
	PropertyRowGenerator->RegisterInstancedCustomPropertyTypeLayout(NAME_Vector4, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FFastColorStructCustomization::MakeInstance), ColorPropertyTypeIdentifier);
	PropertyRowGenerator->RegisterInstancedCustomPropertyTypeLayout(NAME_Vector4f, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FFastColorStructCustomization::MakeInstance), ColorPropertyTypeIdentifier);
	PropertyRowGenerator->RegisterInstancedCustomPropertyTypeLayout(NAME_Vector4d, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FFastColorStructCustomization::MakeInstance), ColorPropertyTypeIdentifier);
}

TArray<TWeakObjectPtr<UObject>> FColorGradingEditorDataModel::GetObjects() const
{
	if (PropertyRowGenerator.IsValid())
	{
		return PropertyRowGenerator->GetSelectedObjects();
	}

	return TArray<TWeakObjectPtr<UObject>>();
}

void FColorGradingEditorDataModel::SetObjects(const TArray<UObject*>& InObjects)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FColorGradingEditorDataModel::SetObjects);

	// Only update the data model if the objects being set are new
	bool bUpdateDataModel = false;
	if (PropertyRowGenerator.IsValid())
	{
		const TArray<TWeakObjectPtr<UObject>>& CurrentObjects = PropertyRowGenerator->GetSelectedObjects();

		if (CurrentObjects.Num() != InObjects.Num())
		{
			bUpdateDataModel = true;
		}
		else
		{
			for (UObject* NewObject : InObjects)
			{
				if (!CurrentObjects.Contains(NewObject))
				{
					bUpdateDataModel = true;
					break;
				}
			}
		}
	}

	if (bUpdateDataModel)
	{
		Reset();

		for (const UObject* Object : InObjects)
		{
			if (Object)
			{
				InitializeDataModelGenerator(Object->GetClass());
			}
		}

		if (PropertyRowGenerator.IsValid())
		{
			PropertyRowGenerator->SetObjects(InObjects);
		}

		SelectedColorGradingGroupIndex = ColorGradingGroups.Num() ? 0 : INDEX_NONE;
		SelectedColorGradingElementIndex = 0;
	}
}

bool FColorGradingEditorDataModel::HasObjectOfType(const UClass* InClass) const
{
	if (PropertyRowGenerator.IsValid())
	{
		const TArray<TWeakObjectPtr<UObject>> SelectedObjects = PropertyRowGenerator->GetSelectedObjects();

		for (const TWeakObjectPtr<UObject>& Object : SelectedObjects)
		{
			if (Object.IsValid() && Object->GetClass()->IsChildOf(InClass))
			{
				return true;
			}
		}
	}

	return false;
}

void FColorGradingEditorDataModel::Reset()
{
	for (const TPair<TWeakObjectPtr<UClass>, TSharedPtr<IColorGradingEditorDataModelGenerator>>& GeneratorInstance : DataModelGeneratorInstances)
	{
		GeneratorInstance.Value->Destroy(SharedThis(this), PropertyRowGenerator.ToSharedRef());
		PropertyRowGenerator->UnregisterInstancedCustomPropertyLayout(GeneratorInstance.Key.Get());
	}

	OnColorGradingGroupDeletedDelegate.Clear();
	OnColorGradingGroupRenamedDelegate.Clear();

	DataModelGeneratorInstances.Empty();
	ColorGradingGroups.Empty();
	SelectedColorGradingGroupIndex = INDEX_NONE;
	SelectedColorGradingElementIndex = INDEX_NONE;
	ColorGradingGroupToolBarWidget = nullptr;
	bShowColorGradingGroupToolBar = false;
}

void FColorGradingEditorDataModel::GetPanelState(FColorGradingPanelState& OutPanelState)
{
	OutPanelState.SelectedColorGradingGroup = SelectedColorGradingGroupIndex;
	OutPanelState.SelectedColorGradingElement = SelectedColorGradingElementIndex;
}

void FColorGradingEditorDataModel::SetPanelState(const FColorGradingPanelState& InPanelState)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FColorGradingEditorDataModel::SetPanelState);

	SelectedColorGradingGroupIndex = InPanelState.SelectedColorGradingGroup;
	SelectedColorGradingElementIndex = InPanelState.SelectedColorGradingElement;

	TArray<UObject*> ObjectsToControl;
	for (const TWeakObjectPtr<UObject>& Object : InPanelState.ControlledObjects)
	{
		if (Object.IsValid())
		{
			ObjectsToControl.Add(Object.Get());
		}
	}

	for (const UObject* Object : ObjectsToControl)
	{
		if (Object)
		{
			InitializeDataModelGenerator(Object->GetClass());
		}
	}

	if (PropertyRowGenerator.IsValid())
	{
		PropertyRowGenerator->SetObjects(ObjectsToControl);
	}

	// After the data model has been created as part of the SetObjects call, check that the saved selected color grading group is still valid,
	// and if not, set the selected group to 0
	if (SelectedColorGradingGroupIndex >= ColorGradingGroups.Num())
	{
		SelectedColorGradingGroupIndex = 0;
	}
}

FColorGradingEditorDataModel::FColorGradingGroup* FColorGradingEditorDataModel::GetSelectedColorGradingGroup()
{
	if (SelectedColorGradingGroupIndex > INDEX_NONE && SelectedColorGradingGroupIndex < ColorGradingGroups.Num())
	{
		return &ColorGradingGroups[SelectedColorGradingGroupIndex];
	}

	return nullptr;
}

void FColorGradingEditorDataModel::SetSelectedColorGradingGroup(int32 InColorGradingGroupIndex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FColorGradingEditorDataModel::SetSelectedColorGradingGroup);

	SelectedColorGradingGroupIndex = ColorGradingGroups.IsValidIndex(InColorGradingGroupIndex) ? InColorGradingGroupIndex : INDEX_NONE;

	// When the color grading group has changed, reset the selected color grading element as well
	const bool bHasColorGradingElements = ColorGradingGroups.IsValidIndex(SelectedColorGradingGroupIndex) && ColorGradingGroups[SelectedColorGradingGroupIndex].ColorGradingElements.Num() > 0;
	SelectedColorGradingElementIndex = bHasColorGradingElements ? 0 : INDEX_NONE;

	OnColorGradingGroupSelectionChangedDelegate.Broadcast();

	// Force the property row generator to rebuild the property node tree, since the data model generators may have made some optimizations
	// based on which color grading group is currently selected
	TArray<UObject*> Objects;
	for (const TWeakObjectPtr<UObject>& WeakObj : PropertyRowGenerator->GetSelectedObjects())
	{
		if (WeakObj.IsValid())
		{
			Objects.Add(WeakObj.Get());
		}
	}

	PropertyRowGenerator->SetObjects(Objects);
}

FColorGradingEditorDataModel::FColorGradingElement* FColorGradingEditorDataModel::GetSelectedColorGradingElement()
{
	if (SelectedColorGradingGroupIndex > INDEX_NONE && SelectedColorGradingGroupIndex < ColorGradingGroups.Num())
	{
		FColorGradingEditorDataModel::FColorGradingGroup& SelectedGroup = ColorGradingGroups[SelectedColorGradingGroupIndex];
		if (SelectedColorGradingElementIndex > INDEX_NONE && SelectedColorGradingElementIndex < SelectedGroup.ColorGradingElements.Num())
		{
			return &SelectedGroup.ColorGradingElements[SelectedColorGradingElementIndex];
		}
	}

	return nullptr;
}

void FColorGradingEditorDataModel::SetSelectedColorGradingElement(int32 InColorGradingElementIndex)
{
	SelectedColorGradingElementIndex = InColorGradingElementIndex;
	OnColorGradingElementSelectionChangedDelegate.Broadcast();
}

void FColorGradingEditorDataModel::InitializeDataModelGenerator(UClass* InClass)
{
	UClass* CurrentClass = InClass;
	while (CurrentClass)
	{
		if (RegisteredDataModelGenerators.Contains(CurrentClass) && !DataModelGeneratorInstances.Contains(CurrentClass))
		{
			if (RegisteredDataModelGenerators[CurrentClass].IsBound())
			{
				TSharedRef<IColorGradingEditorDataModelGenerator> Generator = RegisteredDataModelGenerators[CurrentClass].Execute();
				Generator->Initialize(SharedThis(this), PropertyRowGenerator.ToSharedRef());

				DataModelGeneratorInstances.Add(CurrentClass, Generator);
			}
		}

		CurrentClass = CurrentClass->GetSuperClass();
	}
}

TSharedPtr<IColorGradingEditorDataModelGenerator> FColorGradingEditorDataModel::GetDataModelGenerator(UClass* InClass) const
{
	UClass* CurrentClass = InClass;
	while (CurrentClass)
	{
		if (DataModelGeneratorInstances.Contains(CurrentClass))
		{
			return DataModelGeneratorInstances[CurrentClass];
		}

		CurrentClass = CurrentClass->GetSuperClass();
	}

	return nullptr;
}

void FColorGradingEditorDataModel::OnPropertyRowGeneratorRefreshed()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FColorGradingEditorDataModel::OnPropertyRowGeneratorRefreshed);

 	ColorGradingGroups.Empty();

	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = PropertyRowGenerator->GetSelectedObjects();

	if (SelectedObjects.Num() == 1)
	{
		TWeakObjectPtr<UObject> SelectedObject = SelectedObjects[0];

		if (SelectedObject.IsValid())
		{
			if (TSharedPtr<IColorGradingEditorDataModelGenerator> Generator = GetDataModelGenerator(SelectedObject->GetClass()))
			{
				Generator->GenerateDataModel(*PropertyRowGenerator, *this);
			}
		}
	}

	// TODO: Figure out what needs to be done to support multiple disparate types of objects being color graded at the same time

	OnDataModelGeneratedDelegate.Broadcast();
}

#undef LOCTEXT_NAMESPACE