// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorGradingEditorModule.h"

#include "ColorGradingCommands.h"
#include "ColorGradingEditorDataModel.h"
#include "ColorGradingMixerObjectFilterRegistry.h"
#include "DataModelGenerators/ColorGradingDataModelGenerator_CameraActor.h"
#include "DataModelGenerators/ColorGradingDataModelGenerator_PostProcessVolume.h"
#include "SColorGradingPanel.h"

#include "Camera/CameraActor.h"
#include "CineCameraActor.h"
#include "ColorGradingMixerSelectionInterface.h"
#include "Engine/PostProcessVolume.h"
#include "Framework/Docking/LayoutExtender.h"
#include "LevelEditor.h"
#include "PropertyEditorModule.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "ColorGradingEditor"

const FName FColorGradingEditorModule::ColorGradingPanelTabId(TEXT("ColorGradingPanel"));

void FColorGradingEditorModule::StartupModule()
{
	FColorGradingEditorDataModel::RegisterColorGradingDataModelGenerator<APostProcessVolume>(
		FGetDetailsDataModelGenerator::CreateStatic(&FColorGradingDataModelGenerator_PostProcessVolume::MakeInstance));

	FColorGradingEditorDataModel::RegisterColorGradingDataModelGenerator<ACameraActor>(
		FGetDetailsDataModelGenerator::CreateStatic(&FColorGradingDataModelGenerator_CameraActor::MakeInstance));

	FColorGradingMixerObjectFilterRegistry::RegisterActorClassToPlace(APostProcessVolume::StaticClass());
	FColorGradingMixerObjectFilterRegistry::RegisterActorClassToPlace(ACineCameraActor::StaticClass());
	FColorGradingMixerObjectFilterRegistry::RegisterActorClassToPlace(ACameraActor::StaticClass());

	FColorGradingMixerObjectFilterRegistry::RegisterObjectClassToFilter(APostProcessVolume::StaticClass());
	FColorGradingMixerObjectFilterRegistry::RegisterObjectClassToFilter(ACameraActor::StaticClass());

	FColorGradingCommands::Register();

	FCoreDelegates::OnFEngineLoopInitComplete.AddRaw(this, &FColorGradingEditorModule::OnFEngineLoopInitComplete);
}

void FColorGradingEditorModule::ShutdownModule()
{
	MainPanel.Reset();
}

void FColorGradingEditorModule::OnFEngineLoopInitComplete()
{
	RegisterMenuItem();

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditorModule.OnRegisterLayoutExtensions().AddRaw(this, &FColorGradingEditorModule::RegisterLevelEditorLayout);
}

void FColorGradingEditorModule::RegisterMenuItem()
{
	FTabSpawnerEntry& BrowserSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
		ColorGradingPanelTabId,
		FOnSpawnTab::CreateRaw(this, &FColorGradingEditorModule::SpawnMainPanelTab))
			.SetIcon(FSlateIcon(FColorGradingEditorStyle::Get().GetStyleSetName(), "ColorGrading.ToolbarButton"))
			.SetDisplayName(LOCTEXT("OpenColorGradingPanelMenuItem", "Color Grading"))
			.SetTooltipText(LOCTEXT("OpenColorGradingPanelTooltip", "Open the Color Grading panel, which contains extended controls for color grading"))
			.SetMenuType(ETabSpawnerMenuType::Enabled);

	BrowserSpawnerEntry.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory());
}

TSharedRef<SDockTab> FColorGradingEditorModule::SpawnMainPanelTab(const FSpawnTabArgs& Args)
{
	if (!MainPanel.IsValid())
	{
		SAssignNew(MainPanel, SColorGradingPanel)
		.SelectionInterface(MakeShared<FColorGradingMixerSelectionInterface>());
	}

	const TSharedRef<SDockTab> DockTab = SNew(SDockTab).TabRole(ETabRole::NomadTab);
	DockTab->SetContent(MainPanel.ToSharedRef());

	return DockTab;
}

void FColorGradingEditorModule::RegisterLevelEditorLayout(FLayoutExtender& Extender)
{
	// By default, place the Color Grading panel in the same tab group as the Content Browser
	Extender.ExtendLayout(
		FTabId("ContentBrowserTab1"),
		ELayoutExtensionPosition::Before,
		FTabManager::FTab(FTabId(ColorGradingPanelTabId), ETabState::ClosedTab)
	);
}

IMPLEMENT_MODULE(FColorGradingEditorModule, ColorGradingEditor);

#undef LOCTEXT_NAMESPACE
