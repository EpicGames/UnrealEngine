// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IColorGradingEditor.h"
#include "Widgets/Docking/SDockTab.h"

class FColorGradingEditorDataModel;
class SColorGradingPanel;
class FLayoutExtender;

/**
 * Display Cluster Color Grading module
 */
class FColorGradingEditorModule : public IColorGradingEditor
{
public:
	//~ IColorGradingEditor interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual FName GetColorGradingTabSpawnerId() const override { return ColorGradingPanelTabId; }
	//~ End IColorGradingEditor interface

private:
	/** Called right before the engine starts ticking */
	void OnFEngineLoopInitComplete();

	/** Register the level editor layout extension for the Color Grading panel */
	void RegisterLevelEditorLayout(FLayoutExtender& Extender);

	/** Register the menu item that opens the Color Grading panel */
	void RegisterMenuItem();

	/** Spawn the tab containing the Color Grading panel */
	TSharedRef<SDockTab> SpawnMainPanelTab(const FSpawnTabArgs& Args);

	/** ID to uniquely identify the Color Grading panel tab */
	static const FName ColorGradingPanelTabId;

	/** The main dockable color grading panel */
	TSharedPtr<SColorGradingPanel> MainPanel;
};
