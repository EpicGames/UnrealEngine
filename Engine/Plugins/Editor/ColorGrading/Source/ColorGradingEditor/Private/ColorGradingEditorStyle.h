// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyle.h"
#include "Styling/ToolBarStyle.h"


/** Styleset for the color grading editor UI elements */
class FColorGradingEditorStyle final : public FSlateStyleSet
{
public:

	FColorGradingEditorStyle()
		: FSlateStyleSet("ColorGradingEditorStyle")
	{
		const FVector2D Icon16x16(16.0f, 16.0f);

		SetParentStyleName(FAppStyle::GetAppStyleSetName());

		// Set miscellaneous icons
		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Editor/ColorGrading/Content/Icons/"));
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));

		Set("ColorGrading.ToolbarButton", new IMAGE_BRUSH_SVG("ColorGrading", Icon16x16));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	virtual ~FColorGradingEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	static FColorGradingEditorStyle& Get()
	{
		static FColorGradingEditorStyle Inst;
		return Inst;
	}
};