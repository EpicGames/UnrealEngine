// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorGradingEditorUtil.h"

#include "ColorGradingEditorUtil.h"
#include "ColorGradingEditorStyle.h"
#include "Framework/Docking/TabManager.h"
#include "IColorGradingEditor.h"
#include "LevelEditor.h"
#include "Styling/CoreStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

namespace ColorGradingEditorUtil
{

TSharedRef<SWidget> MakeColorGradingLaunchButton(bool bWrapInBox)
{
	TSharedRef<SWidget> Result = SNew(SButton)
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Center)
	.OnClicked_Lambda([]()
	{
		FLevelEditorModule& LevelEditor = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditor.GetLevelEditorTabManager()->TryInvokeTab(IColorGradingEditor::Get().GetColorGradingTabSpawnerId());
		return FReply::Handled();
	})
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SImage)
			.Image(FColorGradingEditorStyle::Get().GetBrush("ColorGrading.ToolbarButton"))
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4, 0, 0, 0)
		[
			SNew(STextBlock)
			.Visibility(EVisibility::HitTestInvisible)
			.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("ButtonText"))
			.Text(NSLOCTEXT("ColorCorrectWindowDetails", "OpenColorGrading", "Open Color Grading"))

		]
	];

	if (bWrapInBox)
	{
		Result = SNew(SBox)
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.Padding(0, 2)
		[
			Result
		];
	}

	return Result;
}

}
