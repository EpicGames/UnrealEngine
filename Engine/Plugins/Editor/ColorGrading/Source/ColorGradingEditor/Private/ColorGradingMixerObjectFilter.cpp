// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorGradingMixerObjectFilter.h"

#include "ActorTreeItem.h"
#include "ComponentTreeItem.h"
#include "ColorGradingMixerContextObject.h"
#include "ColorGradingMixerObjectFilterRegistry.h"
#include "ISceneOutliner.h"
#include "ToolMenuContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ColorGradingMixerObjectFilter)

TSet<UClass*> UColorGradingMixerObjectFilter::GetObjectClassesToFilter() const
{
	return FColorGradingMixerObjectFilterRegistry::GetObjectClassesToFilter();
}

TSet<TSubclassOf<AActor>> UColorGradingMixerObjectFilter::GetObjectClassesToPlace() const
{
	return FColorGradingMixerObjectFilterRegistry::GetActorClassesToPlace();
}

TArray<AActor*> UColorGradingMixerObjectFilter::FindAssociatedActors(AActor* InActor) const
{
	if (InActor)
	{
		if (const IColorGradingMixerObjectHierarchyConfig* Config = FColorGradingMixerObjectFilterRegistry::GetClassHierarchyConfig(InActor->GetClass()))
		{
			return Config->FindAssociatedActors(InActor);
		}
	}

	return {};
}

TArray<FObjectMixerEditorActorSubObject> UColorGradingMixerObjectFilter::GetActorSubObjects(AActor* InActor) const
{
	if (InActor)
	{
		if (const IColorGradingMixerObjectHierarchyConfig* Config = FColorGradingMixerObjectFilterRegistry::GetClassHierarchyConfig(InActor->GetClass()))
		{
			return Config->GetActorSubObjects(InActor);
		}
	}

	return {};
}


bool UColorGradingMixerObjectFilter::IsActorAssociated(AActor* Actor, AActor* AssociatedActor) const
{
	if (Actor && AssociatedActor)
	{
		if (const IColorGradingMixerObjectHierarchyConfig* Config = FColorGradingMixerObjectFilterRegistry::GetClassHierarchyConfig(Actor->GetClass()))
		{
			return Config->IsActorAssociated(Actor, AssociatedActor);
		}
	}

	return false;
}

bool UColorGradingMixerObjectFilter::HasCustomDropHandling(const ISceneOutlinerTreeItem& DropTarget) const
{
	if (const UObject* TargetObject = GetObjectForTreeItem(DropTarget))
	{
		if (const IColorGradingMixerObjectHierarchyConfig* Config = FColorGradingMixerObjectFilterRegistry::GetClassHierarchyConfig(TargetObject->GetClass()))
		{
			return Config->HasCustomDropHandling();
		}
	}

	return false;
}

FSceneOutlinerDragValidationInfo UColorGradingMixerObjectFilter::ValidateDrop(const ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload) const
{
	if (UObject* TargetObject = GetObjectForTreeItem(DropTarget))
	{
		if (const IColorGradingMixerObjectHierarchyConfig* Config = FColorGradingMixerObjectFilterRegistry::GetClassHierarchyConfig(TargetObject->GetClass()))
		{
			return Config->ValidateDrop(TargetObject, Payload);
		}
	}

	return FSceneOutlinerDragValidationInfo::Invalid();
}

void UColorGradingMixerObjectFilter::OnDrop(ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const
{
	if (UObject* TargetObject = GetObjectForTreeItem(DropTarget))
	{
		if (const IColorGradingMixerObjectHierarchyConfig* Config = FColorGradingMixerObjectFilterRegistry::GetClassHierarchyConfig(TargetObject->GetClass()))
		{
			if (Config->OnDrop(TargetObject, Payload, ValidationInfo))
			{
				if (ISceneOutliner* SceneOutliner = DropTarget.WeakSceneOutliner.Pin().Get())
				{
					SceneOutliner->FullRefresh();
				}
			}
		}
	}
}

TSet<FName> UColorGradingMixerObjectFilter::GetPropertiesThatRequireListRefresh() const
{
	TSet<FName> PropertyNames;

	for (UClass* Class : FColorGradingMixerObjectFilterRegistry::GetObjectClassesToFilter())
	{
		if (const IColorGradingMixerObjectHierarchyConfig* Config = FColorGradingMixerObjectFilterRegistry::GetClassHierarchyConfig(Class))
		{
			PropertyNames.Append(Config->GetPropertiesThatRequireListRefresh());
		}
	}

	return MoveTemp(PropertyNames);
}

void UColorGradingMixerObjectFilter::OnContextMenuContextCreated(FToolMenuContext& Context) const
{
	Context.AddObject(NewObject<UColorGradingMixerContextObject>());
}

UObject* UColorGradingMixerObjectFilter::GetObjectForTreeItem(const ISceneOutlinerTreeItem& TreeItem) const
{
	if (const FActorTreeItem* ActorTreeItem = TreeItem.CastTo<FActorTreeItem>())
	{
		return ActorTreeItem->Actor.Get();
	}

	if (const FComponentTreeItem* ComponentTreeItem = TreeItem.CastTo<FComponentTreeItem>())
	{
		return ComponentTreeItem->Component.Get();
	}

	return nullptr;
}
