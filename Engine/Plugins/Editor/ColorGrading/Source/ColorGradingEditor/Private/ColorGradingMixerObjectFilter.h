// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectFilter/ObjectMixerEditorObjectFilter.h"

#include "ColorGradingMixerObjectFilter.generated.h"

#define UE_API COLORGRADINGEDITOR_API

UCLASS(MinimalAPI, BlueprintType, EditInlineNew)
class UColorGradingMixerObjectFilter : public UObjectMixerObjectFilter
{
	GENERATED_BODY()
public:

	UE_API virtual TSet<UClass*> GetObjectClassesToFilter() const override;
	UE_API virtual TSet<TSubclassOf<AActor>> GetObjectClassesToPlace() const override;
	UE_API virtual TArray<AActor*> FindAssociatedActors(AActor* InActor) const override;
	UE_API virtual TArray<FObjectMixerEditorActorSubObject> GetActorSubObjects(AActor* InActor) const override;
	UE_API virtual bool IsActorAssociated(AActor* Actor, AActor* AssociatedActor) const override;
	UE_API virtual bool HasCustomDropHandling(const ISceneOutlinerTreeItem& DropTarget) const override;
	UE_API virtual FSceneOutlinerDragValidationInfo ValidateDrop(const ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload) const override;
	UE_API virtual void OnDrop(ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const override;
	UE_API virtual TSet<FName> GetPropertiesThatRequireListRefresh() const override;
	UE_API virtual void OnContextMenuContextCreated(FToolMenuContext& Context) const override;

	virtual bool GetShowTransientObjects() const override
	{
		return true;
	}

	virtual TSet<FName> GetColumnsToShowByDefault() const override
	{
		return {};
	}

	virtual TSet<FName> GetColumnsToExclude() const
	{
		return {};
	}

	virtual TSet<FName> GetForceAddedColumns() const override
	{
		return {};
	}

	virtual bool ShouldIncludeUnsupportedProperties() const override
	{
		return false;
	}

	virtual bool ShouldAllowHybridRows() const override
	{
		// Disabled because both actors and components can have color grading settings, so they must be displayed on separate rows
		// to disambiguate which one is selected for editing. If hybrid mode is enabled, they would be folded into the same row,
		// making selection of the component impossible from the Color Grading panel.
		return false;
	}

	virtual bool ShouldAllowColumnCustomizationByUser() const override
	{
		return false;
	}

	virtual EObjectMixerInheritanceInclusionOptions GetObjectMixerPropertyInheritanceInclusionOptions() const override
	{
		return EObjectMixerInheritanceInclusionOptions::IncludeAllParentsAndChildren;
	}

	virtual EObjectMixerInheritanceInclusionOptions GetObjectMixerPlacementClassInclusionOptions() const override
	{
		return EObjectMixerInheritanceInclusionOptions::None;
	}

private:
	/** Given a tree item, get the object it represents */
	UE_API UObject* GetObjectForTreeItem(const ISceneOutlinerTreeItem& TreeItem) const;
};

#undef UE_API
