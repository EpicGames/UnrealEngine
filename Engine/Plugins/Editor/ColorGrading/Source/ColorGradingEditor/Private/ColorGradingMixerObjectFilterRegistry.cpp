// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorGradingMixerObjectFilterRegistry.h"

const IColorGradingMixerObjectHierarchyConfig* FColorGradingMixerObjectFilterRegistry::GetClassHierarchyConfig(TSubclassOf<AActor> Class)
{
	if (TSharedPtr<IColorGradingMixerObjectHierarchyConfig>* Config = HierarchyConfigInstances.Find(Class))
	{
		return Config->Get();
	}

	// Find the config for this class or its ancestors
	while (Class)
	{
		if (FGetObjectHierarchyConfig* ConfigDelegate = HierarchyConfigs.Find(Class))
		{
			TSharedRef<IColorGradingMixerObjectHierarchyConfig> Config = ConfigDelegate->Execute();
			HierarchyConfigInstances.Emplace(Class, Config);

			return Config.ToSharedPtr().Get();
		}

		Class = Class->GetSuperClass();
	}

	// No config found. Cache nullptr so we don't have to look this up again
	HierarchyConfigInstances.Add(Class, nullptr);
	return nullptr;
}

TArray<TSharedPtr<IColorGradingMixerSelectionHandler>> FColorGradingMixerObjectFilterRegistry::CreateSelectionHandlers()
{
	TArray<TSharedPtr<IColorGradingMixerSelectionHandler>> Handlers;

	for (const FGetSelectionHandler& CreateHandlerDelegate : SelectionHandlers)
	{
		if (CreateHandlerDelegate.IsBound())
		{
			Handlers.Add(CreateHandlerDelegate.Execute());
		}
	}

	return Handlers;
}
