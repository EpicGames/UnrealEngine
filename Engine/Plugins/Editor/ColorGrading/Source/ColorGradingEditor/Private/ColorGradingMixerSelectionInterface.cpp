// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorGradingMixerSelectionInterface.h"

#include "ColorGradingMixerObjectFilterRegistry.h"

FColorGradingMixerSelectionInterface::FColorGradingMixerSelectionInterface()
{
	SelectionHandlers = FColorGradingMixerObjectFilterRegistry::CreateSelectionHandlers();
	for (const TSharedPtr<IColorGradingMixerSelectionHandler>& Handler : SelectionHandlers)
	{
		Handler->OnSelectionChanged.BindRaw(this, &FColorGradingMixerSelectionInterface::OnEditorSelectionChanged);
	}
}

void FColorGradingMixerSelectionInterface::SelectSubObjects(const TArray<UObject*>& InSelectedSubObjects, bool bShouldSelect, bool bSelectEvenIfHidden)
{
	for (const TSharedPtr<IColorGradingMixerSelectionHandler>& Handler : SelectionHandlers)
	{
		TArray<UObject*> Subselection;
		for (UObject* Object : InSelectedSubObjects)
		{
			if (Handler->CanSelectSubObject(Object))
			{
				Subselection.Add(Object);
			}
		}

		Handler->SelectSubObjects(Subselection, bShouldSelect, bSelectEvenIfHidden);
	}
}

TArray<UObject*> FColorGradingMixerSelectionInterface::GetSelectedSubObjects() const
{
	TArray<UObject*> SelectedSubObjects;
	
	for (const TSharedPtr<IColorGradingMixerSelectionHandler>& Handler : SelectionHandlers)
	{
		SelectedSubObjects.Append(Handler->GetSelectedSubObjectsInEditor());
	}
	
	return SelectedSubObjects;
}

void FColorGradingMixerSelectionInterface::OnEditorSelectionChanged()
{
	OnSelectionChanged().Broadcast();
}
