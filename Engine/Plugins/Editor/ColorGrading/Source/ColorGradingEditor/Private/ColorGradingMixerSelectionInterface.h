// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SelectionInterface/LevelEditorObjectMixerSelectionInterface.h"

struct IColorGradingMixerSelectionHandler;

/** Custom selection interface for the color grading drawer's mixer to provide hooks for receiving selection events for actor sub-objects */
class FColorGradingMixerSelectionInterface : public FLevelEditorObjectMixerSelectionInterface
{
public:
	FColorGradingMixerSelectionInterface();
	
	//~ Begin IObjectMixerSelectionInterface interface
	virtual void SelectSubObjects(const TArray<UObject*>& InSelectedSubObjects, bool bShouldSelect, bool bSelectEvenIfHidden) override;
	virtual TArray<UObject*> GetSelectedSubObjects() const override;
	//~ End IObjectMixerSelectionInterface interface

private:
	void OnEditorSelectionChanged();
	
private:
	/** A list of intantiated selection handlers that will process selection of actor sub-objects */
	TArray<TSharedPtr<IColorGradingMixerSelectionHandler>> SelectionHandlers;
};
