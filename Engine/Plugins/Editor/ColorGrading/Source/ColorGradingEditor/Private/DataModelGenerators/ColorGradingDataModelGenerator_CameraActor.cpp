// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorGradingDataModelGenerator_CameraActor.h"

#include "ClassIconFinder.h"
#include "Camera/CameraActor.h"
#include "Camera/CameraComponent.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailCustomization.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "PropertyHandle.h"
#include "PropertyPathHelpers.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ColorGradingEditor"

// Modify a property handle so that it creates a transaction when the property changes
TSharedPtr<IPropertyHandle> MakePropertyTransactional(const TSharedPtr<IPropertyHandle>& PropertyHandle)
{
	if (PropertyHandle.IsValid())
	{
		PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([PropertyHandle]
		{
			TArray<UObject*> OuterObjects;
			PropertyHandle->GetOuterObjects(OuterObjects);
			for (UObject* Object : OuterObjects)
			{
				if (!Object)
				{
					continue;
				}

				if (!Object->HasAnyFlags(RF_Transactional))
				{
					Object->SetFlags(RF_Transactional);
				}

				SaveToTransactionBuffer(Object, false);
				SnapshotTransactionBuffer(Object);
			}
		}));
	}

	return PropertyHandle;
}

TSharedRef<IColorGradingEditorDataModelGenerator> FColorGradingDataModelGenerator_CameraActor::MakeInstance()
{
	return MakeShareable(new FColorGradingDataModelGenerator_CameraActor());
}

class FCameraComponentCustomization : public IDetailCustomization
{
public:
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		TArray<FName> Categories;
		DetailBuilder.GetCategoryNames(Categories);

		for (const FName& Category : Categories)
		{
			DetailBuilder.HideCategory(Category);
		}

		// TransformCommon is a custom category that doesn't get returned by GetCategoryNames that also needs to be hidden
		DetailBuilder.HideCategory(TEXT("TransformCommon"));

		IDetailCategoryBuilder& ElementsCategoryBuilder = DetailBuilder.EditCategory(TEXT("ColorGradingElements"));

		TSharedRef<IPropertyHandle> PostProcessSettingsHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCameraComponent, PostProcessSettings));

		// Add elements used to control color grading wheels, which won't be directly shown
		{
			uint32 NumChildren;
			PostProcessSettingsHandle->GetNumChildren(NumChildren);
			for (uint32 Index = 0; Index < NumChildren; ++Index)
			{
				TSharedPtr<IPropertyHandle> ChildHandle = PostProcessSettingsHandle->GetChildHandle(Index);

				if (ChildHandle && ChildHandle->HasMetaData(TEXT("ColorGradingMode")))
				{
					ElementsCategoryBuilder.AddProperty(ChildHandle);
				}
			}
		}

		// Add properties which will be visible in the details panel
		{
			IDetailCategoryBuilder& DetailExposureCategoryBuilder = DetailBuilder.EditCategory(TEXT("DetailView_Exposure"), LOCTEXT("DetailView_ExposureDisplayName", "Exposure"));
			DetailExposureCategoryBuilder.AddProperty(MakePropertyTransactional(PostProcessSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPostProcessSettings, AutoExposureBias))));

			IDetailCategoryBuilder& DetailColorGradingCategoryBuilder = DetailBuilder.EditCategory(TEXT("DetailView_ColorGrading"), LOCTEXT("DetailView_ColorGradingDisplayName", "Color Grading"));
			DetailColorGradingCategoryBuilder.AddProperty(MakePropertyTransactional(PostProcessSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPostProcessSettings, ColorCorrectionShadowsMax))));
			DetailColorGradingCategoryBuilder.AddProperty(MakePropertyTransactional(PostProcessSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPostProcessSettings, ColorCorrectionHighlightsMin))));
			DetailColorGradingCategoryBuilder.AddProperty(MakePropertyTransactional(PostProcessSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPostProcessSettings, ColorCorrectionHighlightsMax))));

			IDetailCategoryBuilder& DetailWhiteBalanceCategoryBuilder = DetailBuilder.EditCategory(TEXT("DetailView_WhiteBalance"), LOCTEXT("DetailView_WhiteBalanceDisplayName", "White Balance"));
			DetailWhiteBalanceCategoryBuilder.AddProperty(PostProcessSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPostProcessSettings, TemperatureType)));
			DetailWhiteBalanceCategoryBuilder.AddProperty(MakePropertyTransactional(PostProcessSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPostProcessSettings, WhiteTemp))));
			DetailWhiteBalanceCategoryBuilder.AddProperty(MakePropertyTransactional(PostProcessSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPostProcessSettings, WhiteTint))));

			IDetailCategoryBuilder& DetailMiscCategoryBuilder = DetailBuilder.EditCategory(TEXT("DetailView_Misc"), LOCTEXT("DetailView_MiscDisplayName", "Misc"));
			DetailMiscCategoryBuilder.AddProperty(MakePropertyTransactional(PostProcessSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPostProcessSettings, BlueCorrection))));
			DetailMiscCategoryBuilder.AddProperty(MakePropertyTransactional(PostProcessSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPostProcessSettings, ExpandGamut))));
			DetailMiscCategoryBuilder.AddProperty(PostProcessSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPostProcessSettings, SceneColorTint)));
		}

		DetailBuilder.SortCategories([](const TMap<FName, IDetailCategoryBuilder*>& CategoryMap)
		{
			const TMap<FName, int32> SortOrder =
			{
				{ TEXT("DetailView_Exposure"), 0},
				{ TEXT("DetailView_ColorGrading"), 1},
				{ TEXT("DetailView_WhiteBalance"), 2},
				{ TEXT("DetailView_Misc"), 3}
			};

			for (const TPair<FName, int32>& SortPair : SortOrder)
			{
				if (CategoryMap.Contains(SortPair.Key))
				{
					CategoryMap[SortPair.Key]->SetSortOrder(SortPair.Value);
				}
			}
		});
	}
};

void FColorGradingDataModelGenerator_CameraActor::Initialize(const TSharedRef<class FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator)
{
	PropertyRowGenerator->RegisterInstancedCustomPropertyLayout(UCameraComponent::StaticClass(), FOnGetDetailCustomizationInstance::CreateLambda([]
	{
		return MakeShared<FCameraComponentCustomization>();
	}));
}

void FColorGradingDataModelGenerator_CameraActor::Destroy(const TSharedRef<class FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator)
{
	PropertyRowGenerator->UnregisterInstancedCustomPropertyLayout(UCameraComponent::StaticClass());
}

void FColorGradingDataModelGenerator_CameraActor::GenerateDataModel(IPropertyRowGenerator& PropertyRowGenerator, FColorGradingEditorDataModel& OutColorGradingDataModel)
{
	// Generate a list of camera components on the selected actors
	TArray<TWeakObjectPtr<UCameraComponent>> SelectedCameras;
	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = PropertyRowGenerator.GetSelectedObjects();

	for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
	{
		if (SelectedObject.IsValid() && SelectedObject->IsA<ACameraActor>())
		{
			TWeakObjectPtr<ACameraActor> SelectedActor = CastChecked<ACameraActor>(SelectedObject.Get());
			if (UCameraComponent* SelectedComponent = SelectedActor->GetComponentByClass<UCameraComponent>())
			{
				SelectedCameras.Add(SelectedComponent);
			}
		}
	}

	if (!SelectedCameras.Num())
	{
		return;
	}

	// Get the category containing elements for controlling the color grading wheels
	const TArray<TSharedRef<IDetailTreeNode>>& RootNodes = PropertyRowGenerator.GetRootTreeNodes();
	const TSharedRef<IDetailTreeNode>* ColorGradingElementsPtr = RootNodes.FindByPredicate([](const TSharedRef<IDetailTreeNode>& Node)
	{
		return Node->GetNodeName() == TEXT("ColorGradingElements");
	});

	if (!ColorGradingElementsPtr)
	{
		return;
	}

	// Display the custom details categories we built earlier
	FColorGradingEditorDataModel::FColorGradingGroup ColorGradingGroup;
	ColorGradingGroup.DetailsViewCategories =
	{
		TEXT("DetailView_Exposure"),
		TEXT("DetailView_ColorGrading"),
		TEXT("DetailView_WhiteBalance"),
		TEXT("DetailView_Misc")
	};

	// Add color grading properties, assigning them to groups and elements based on their metadata
	const TSharedRef<IDetailTreeNode> ColorGradingElementsNode = *ColorGradingElementsPtr;
	TArray<TSharedRef<IDetailTreeNode>> ColorGradingPropertyNodes;
	ColorGradingElementsNode->GetChildren(ColorGradingPropertyNodes);

	TMap<FString, FColorGradingEditorDataModel::FColorGradingElement> ColorGradingElements;

	for (const TSharedRef<IDetailTreeNode>& PropertyNode : ColorGradingPropertyNodes)
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = PropertyNode->CreatePropertyHandle();

		if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
		{
			FString CategoryName = TEXT("");
			FString GroupName = TEXT("");
			PropertyHandle->GetDefaultCategoryName().ToString().Split(TEXT("|"), &CategoryName, &GroupName);

			if (!ColorGradingElements.Contains(GroupName))
			{
				FColorGradingEditorDataModel::FColorGradingElement& ColorGradingElement = ColorGradingElements.Add(GroupName);
				ColorGradingElement.DisplayName = FText::FromString(GroupName);
			}

			AddPropertyToColorGradingElement(PropertyHandle, ColorGradingElements[GroupName]);
		}
	}

	ColorGradingElements.GenerateValueArray(ColorGradingGroup.ColorGradingElements);
	
	// Generate layout
	const TWeakObjectPtr<ACameraActor> FirstCameraActor = Cast<ACameraActor>(SelectedCameras[0]->GetAttachmentRootActor());

	ColorGradingGroup.GroupHeaderWidget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(0, 1, 6, 1))
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.WidthOverride(16)
			.HeightOverride(16)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FClassIconFinder::FindIconForActor(FirstCameraActor))
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(STextBlock)
			.Text(MakeAttributeLambda([WeakActor = FirstCameraActor]()
			{
				return WeakActor.IsValid()
					? FText::FromString(WeakActor->GetActorLabel())
					: FText::GetEmpty();
			}))
			.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
		];

	OutColorGradingDataModel.ColorGradingGroups.Add(ColorGradingGroup);
}

void FColorGradingDataModelGenerator_CameraActor::AddPropertyToColorGradingElement(const TSharedPtr<IPropertyHandle>& PropertyHandle, FColorGradingEditorDataModel::FColorGradingElement& ColorGradingElement)
{
	const FString ColorGradingModeString = PropertyHandle->GetProperty()->GetMetaData(TEXT("ColorGradingMode")).ToLower();

	if (!ColorGradingModeString.IsEmpty())
	{
		if (ColorGradingModeString.Compare(TEXT("saturation")) == 0)
		{
			ColorGradingElement.SaturationPropertyHandle = PropertyHandle;
		}
		else if (ColorGradingModeString.Compare(TEXT("contrast")) == 0)
		{
			ColorGradingElement.ContrastPropertyHandle = PropertyHandle;
		}
		else if (ColorGradingModeString.Compare(TEXT("gamma")) == 0)
		{
			ColorGradingElement.GammaPropertyHandle = PropertyHandle;
		}
		else if (ColorGradingModeString.Compare(TEXT("gain")) == 0)
		{
			ColorGradingElement.GainPropertyHandle = PropertyHandle;
		}
		else if (ColorGradingModeString.Compare(TEXT("offset")) == 0)
		{
			ColorGradingElement.OffsetPropertyHandle = PropertyHandle;
		}
	}
}

#undef LOCTEXT_NAMESPACE