// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ColorGradingEditorDataModel.h"

class IDetailTreeNode;

/** Color Grading Data Model Generator for the ACameraActor actor class */
class FColorGradingDataModelGenerator_CameraActor: public IColorGradingEditorDataModelGenerator
{
public:
	static TSharedRef<IColorGradingEditorDataModelGenerator> MakeInstance();

	//~ IColorGradingDataModelGenerator interface
	virtual void Initialize(const TSharedRef<class FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator) override;
	virtual void Destroy(const TSharedRef<class FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator) override;
	virtual void GenerateDataModel(IPropertyRowGenerator& PropertyRowGenerator, FColorGradingEditorDataModel& OutColorGradingDataModel) override;
	//~ End IColorGradingDataModelGenerator interface

private:
	// Add a color grading property to the a color grading element, assigning it to the appropriate variable based on its metadata
	void AddPropertyToColorGradingElement(const TSharedPtr<IPropertyHandle>& PropertyHandle, FColorGradingEditorDataModel::FColorGradingElement& ColorGradingElement);
};