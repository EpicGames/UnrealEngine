// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorGradingDataModelGenerator_PostProcessVolume.h"

#include "ClassIconFinder.h"
#include "Engine/PostProcessVolume.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailCustomization.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "PropertyHandle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ColorGradingEditor"

TSharedRef<IColorGradingEditorDataModelGenerator> FColorGradingDataModelGenerator_PostProcessVolume::MakeInstance()
{
	return MakeShareable(new FColorGradingDataModelGenerator_PostProcessVolume());
}

class FPostProcessVolumeCustomization : public IDetailCustomization
{
public:
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		TArray<FName> Categories;
		DetailBuilder.GetCategoryNames(Categories);

		for (const FName& Category : Categories)
		{
			DetailBuilder.HideCategory(Category);
		}

		// TransformCommon is a custom category that doesn't get returned by GetCategoryNames that also needs to be hidden
		DetailBuilder.HideCategory(TEXT("TransformCommon"));

		TSharedRef<IPropertyHandle> SettingsHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(APostProcessVolume, Settings));

		uint32 NumChildren;
		SettingsHandle->GetNumChildren(NumChildren);
		for (uint32 Index = 0; Index < NumChildren; ++Index)
		{
			TSharedPtr<IPropertyHandle> ChildHandle = SettingsHandle->GetChildHandle(Index);

			FString CategoryName = TEXT("");
			FString GroupName = TEXT("");
			ChildHandle->GetDefaultCategoryName().ToString().Split(TEXT("|"), &CategoryName, &GroupName);

			if (CategoryName == "Color Grading")
			{
				if (ChildHandle->HasMetaData(TEXT("ColorGradingMode")))
				{
					IDetailCategoryBuilder& ColorGradingCategory = DetailBuilder.EditCategory(FName("ColorGradingElements"));
					ColorGradingCategory.AddProperty(ChildHandle);
				}
				else
				{
					IDetailCategoryBuilder& GroupCategory = DetailBuilder.EditCategory(FName("DetailView_" + GroupName), FText::FromString(GroupName));
					GroupCategory.AddProperty(ChildHandle);
				}
			}
			else if (ChildHandle->GetProperty()->GetNameCPP() == GET_MEMBER_NAME_STRING_CHECKED(FPostProcessSettings, AutoExposureBias))
			{
				// UE-192122: Also add Exposure compensation property to details view, to match DCRA properties
				IDetailCategoryBuilder& GroupCategory = DetailBuilder.EditCategory(FName("DetailView_" + GroupName), FText::FromString(GroupName), ECategoryPriority::Important);
				GroupCategory.AddProperty(ChildHandle);
			}
		}
	}
};

void FColorGradingDataModelGenerator_PostProcessVolume::Initialize(const TSharedRef<class FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator)
{
	PropertyRowGenerator->RegisterInstancedCustomPropertyLayout(APostProcessVolume::StaticClass(), FOnGetDetailCustomizationInstance::CreateLambda([]
	{
		return MakeShared<FPostProcessVolumeCustomization>();
	}));
}

void FColorGradingDataModelGenerator_PostProcessVolume::Destroy(const TSharedRef<class FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator)
{
	PropertyRowGenerator->UnregisterInstancedCustomPropertyLayout(APostProcessVolume::StaticClass());
}

void FColorGradingDataModelGenerator_PostProcessVolume::GenerateDataModel(IPropertyRowGenerator& PropertyRowGenerator, FColorGradingEditorDataModel& OutColorGradingDataModel)
{
	TArray<TWeakObjectPtr<APostProcessVolume>> SelectedPPVs;
	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = PropertyRowGenerator.GetSelectedObjects();
	for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
	{
		if (SelectedObject.IsValid() && SelectedObject->IsA<APostProcessVolume>())
		{
			TWeakObjectPtr<APostProcessVolume> SelectedRootActor = CastChecked<APostProcessVolume>(SelectedObject.Get());
			SelectedPPVs.Add(SelectedRootActor);
		}
	}

	if (!SelectedPPVs.Num())
	{
		return;
	}

	const TArray<TSharedRef<IDetailTreeNode>>& RootNodes = PropertyRowGenerator.GetRootTreeNodes();

	const TSharedRef<IDetailTreeNode>* ColorGradingElementsNodePtr = RootNodes.FindByPredicate([](const TSharedRef<IDetailTreeNode>& Node)
	{
		return Node->GetNodeName() == TEXT("ColorGradingElements");
	});

	if (ColorGradingElementsNodePtr)
	{
		const TSharedRef<IDetailTreeNode> ColorGradingElementsNode = *ColorGradingElementsNodePtr;
		FColorGradingEditorDataModel::FColorGradingGroup ColorGradingGroup;

		TArray<TSharedRef<IDetailTreeNode>> PropertyGroupNodes;
		ColorGradingElementsNode->GetChildren(PropertyGroupNodes);

		TMap<FString, FColorGradingEditorDataModel::FColorGradingElement> ColorGradingElements;

		for (const TSharedRef<IDetailTreeNode>& PropertyGroupNode : PropertyGroupNodes)
		{
			TSharedPtr<IPropertyHandle> PropertyHandle = PropertyGroupNode->CreatePropertyHandle();

			if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
			{
				FString CategoryName = TEXT("");
				FString GroupName = TEXT("");
				PropertyHandle->GetDefaultCategoryName().ToString().Split(TEXT("|"), &CategoryName, &GroupName);

				if (!ColorGradingElements.Contains(GroupName))
				{
					FColorGradingEditorDataModel::FColorGradingElement& ColorGradingElement = ColorGradingElements.Add(GroupName);
					ColorGradingElement.DisplayName = FText::FromString(GroupName);
				}

				AddPropertyToColorGradingElement(PropertyHandle, ColorGradingElements[GroupName]);
			}
		}

		ColorGradingElements.GenerateValueArray(ColorGradingGroup.ColorGradingElements);

		// Add all categories that are not the color grading elements category to the list of categories to display in the detail view
		for (const TSharedRef<IDetailTreeNode>& Node : RootNodes)
		{
			FName NodeName = Node->GetNodeName();
			if (NodeName != TEXT("ColorGradingElements"))
			{
				ColorGradingGroup.DetailsViewCategories.Add(NodeName);
			}
		}

		ColorGradingGroup.GroupHeaderWidget = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0, 1, 6, 1))
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.WidthOverride(16)
				.HeightOverride(16)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FClassIconFinder::FindIconForActor(SelectedPPVs[0]))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Text(MakeAttributeLambda([WeakActor = SelectedPPVs[0]]()
				{
					return WeakActor.IsValid()
						? FText::FromString(WeakActor->GetActorLabel())
						: FText::GetEmpty();
				}))
				.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
			];

		OutColorGradingDataModel.ColorGradingGroups.Add(ColorGradingGroup);
	}
}

void FColorGradingDataModelGenerator_PostProcessVolume::AddPropertyToColorGradingElement(const TSharedPtr<IPropertyHandle>& PropertyHandle, FColorGradingEditorDataModel::FColorGradingElement& ColorGradingElement)
{
	const FString ColorGradingModeString = PropertyHandle->GetProperty()->GetMetaData(TEXT("ColorGradingMode")).ToLower();

	if (!ColorGradingModeString.IsEmpty())
	{
		if (ColorGradingModeString.Compare(TEXT("saturation")) == 0)
		{
			ColorGradingElement.SaturationPropertyHandle = PropertyHandle;
		}
		else if (ColorGradingModeString.Compare(TEXT("contrast")) == 0)
		{
			ColorGradingElement.ContrastPropertyHandle = PropertyHandle;
		}
		else if (ColorGradingModeString.Compare(TEXT("gamma")) == 0)
		{
			ColorGradingElement.GammaPropertyHandle = PropertyHandle;
		}
		else if (ColorGradingModeString.Compare(TEXT("gain")) == 0)
		{
			ColorGradingElement.GainPropertyHandle = PropertyHandle;
		}
		else if (ColorGradingModeString.Compare(TEXT("offset")) == 0)
		{
			ColorGradingElement.OffsetPropertyHandle = PropertyHandle;
		}
	}
}

#undef LOCTEXT_NAMESPACE