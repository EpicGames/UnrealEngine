// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ColorGradingEditorDataModel.h"

class IDetailTreeNode;

/** Color Grading Data Model Generator for the APostProcessVolume actor class */
class FColorGradingDataModelGenerator_PostProcessVolume: public IColorGradingEditorDataModelGenerator
{
public:
	static TSharedRef<IColorGradingEditorDataModelGenerator> MakeInstance();

	//~ IColorGradingDataModelGenerator interface
	virtual void Initialize(const TSharedRef<class FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator) override;
	virtual void Destroy(const TSharedRef<class FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator) override;
	virtual void GenerateDataModel(IPropertyRowGenerator& PropertyRowGenerator, FColorGradingEditorDataModel& OutColorGradingDataModel) override;
	//~ End IColorGradingDataModelGenerator interface

private:
	void AddPropertyToColorGradingElement(const TSharedPtr<IPropertyHandle>& PropertyHandle, FColorGradingEditorDataModel::FColorGradingElement& ColorGradingElement);
};