// Copyright Epic Games, Inc. All Rights Reserved.

#include "FColorGradingDetailTreeItem.h"

#include "DetailTreeNode.h"
#include "Misc/App.h"

void FColorGradingDetailTreeItem::Initialize(const FOnFilterDetailTreeNode& NodeFilter)
{
	if (DetailTreeNode.IsValid())
	{
		PropertyHandle = DetailTreeNode.Pin()->CreatePropertyHandle();

		TArray<TSharedRef<IDetailTreeNode>> ChildNodes;
		DetailTreeNode.Pin()->GetChildren(ChildNodes);

		for (const TSharedRef<IDetailTreeNode>& ChildNode : ChildNodes)
		{
			bool bShouldDisplayNode = true;
			if (NodeFilter.IsBound())
			{
				bShouldDisplayNode = NodeFilter.Execute(ChildNode);
			}

			if (bShouldDisplayNode)
			{
				TSharedPtr<FDetailTreeNode> CastChildNode = StaticCastSharedRef<FDetailTreeNode>(ChildNode);
				TSharedRef<FColorGradingDetailTreeItem> ChildItem = MakeShared<FColorGradingDetailTreeItem>(CastChildNode);
				ChildItem->Parent = SharedThis(this);
				ChildItem->Initialize(NodeFilter);

				Children.Add(ChildItem);
			}
		}
	}
}

void FColorGradingDetailTreeItem::GetChildren(TArray<TSharedRef<FColorGradingDetailTreeItem>>& OutChildren) const
{
	OutChildren.Reset(Children.Num());
	OutChildren.Append(Children);
}

TWeakPtr<IDetailTreeNode> FColorGradingDetailTreeItem::GetDetailTreeNode() const
{
	return DetailTreeNode;
}

FName FColorGradingDetailTreeItem::GetNodeName() const
{
	if (DetailTreeNode.IsValid())
	{
		return DetailTreeNode.Pin()->GetNodeName();
	}

	return NAME_None;
}

bool FColorGradingDetailTreeItem::ShouldBeExpanded() const
{
	if (DetailTreeNode.IsValid())
	{
		return DetailTreeNode.Pin()->ShouldBeExpanded();
	}

	return false;
}

void FColorGradingDetailTreeItem::OnItemExpansionChanged(bool bIsExpanded, bool bShouldSaveState)
{
	if (DetailTreeNode.IsValid())
	{
		DetailTreeNode.Pin()->OnItemExpansionChanged(bIsExpanded, bShouldSaveState);
	}
}

bool FColorGradingDetailTreeItem::IsResetToDefaultVisible() const
{
	if (PropertyHandle.IsValid())
	{
		if (PropertyHandle->HasMetaData("NoResetToDefault") || PropertyHandle->GetInstanceMetaData("NoResetToDefault"))
		{
			return false;
		}

		return PropertyHandle->CanResetToDefault();
	}

	return false;
}

void FColorGradingDetailTreeItem::ResetToDefault()
{
	if (PropertyHandle.IsValid())
	{
		PropertyHandle->ResetToDefault();
	}
}

TAttribute<bool> FColorGradingDetailTreeItem::IsPropertyEditingEnabled() const
{
	if (DetailTreeNode.IsValid())
	{
		return DetailTreeNode.Pin()->IsPropertyEditingEnabled();
	}

	return false;
}

bool FColorGradingDetailTreeItem::IsCategory() const
{
	if (DetailTreeNode.IsValid())
	{
		if (DetailTreeNode.Pin()->GetNodeType() == EDetailNodeType::Category)
		{
			return true;
		}
	}

	return false;
}

bool FColorGradingDetailTreeItem::IsItem() const
{
	if (DetailTreeNode.IsValid())
	{
		return DetailTreeNode.Pin()->GetNodeType() == EDetailNodeType::Item;
	}

	return false;
}

bool FColorGradingDetailTreeItem::IsReorderable() const
{
	if (PropertyHandle.IsValid())
	{
		if (TSharedPtr<IPropertyHandle> ParentHandle = PropertyHandle->GetParentHandle())
		{
			const bool bIsParentAnArray = ParentHandle->AsArray().IsValid();
			const bool bIsParentArrayReorderable = !ParentHandle->HasMetaData(TEXT("EditFixedOrder")) && !ParentHandle->HasMetaData(TEXT("ArraySizeEnum"));

			return bIsParentAnArray && bIsParentArrayReorderable && !PropertyHandle->IsEditConst() && !FApp::IsGame();
		}
	}

	return false;
}

bool FColorGradingDetailTreeItem::IsCopyable() const
{
	if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
	{
		static const FName DisableCopyPasteMetaDataName("DisableCopyPaste");

		// Check to see if this property or any of its parents have the DisableCopyPaste metadata
		TSharedPtr<IPropertyHandle> CurrentPropertyHandle = PropertyHandle;
		while (CurrentPropertyHandle.IsValid())
		{
			if (CurrentPropertyHandle->HasMetaData(DisableCopyPasteMetaDataName))
			{
				return false;
			}

			CurrentPropertyHandle = CurrentPropertyHandle->GetParentHandle();
		}

		return true;
	}

	return false;
}

void FColorGradingDetailTreeItem::GenerateDetailWidgetRow(FDetailWidgetRow& OutDetailWidgetRow) const
{
	if (DetailTreeNode.IsValid())
	{
		DetailTreeNode.Pin()->GenerateStandaloneWidget(OutDetailWidgetRow);
	}
}