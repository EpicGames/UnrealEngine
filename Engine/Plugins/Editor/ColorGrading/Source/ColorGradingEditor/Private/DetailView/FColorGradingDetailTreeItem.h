// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DetailView/SColorGradingDetailView.h"

class IDetailTreeNode;
class IPropertyHandle;
class FDetailTreeNode;
class FDetailWidgetRow;

/** A wrapper used to abstract FDetailTreeNode, which is a private abstract class defined in PropertyEditor/Private */
class FColorGradingDetailTreeItem : public TSharedFromThis<FColorGradingDetailTreeItem>
{
public:
	FColorGradingDetailTreeItem(const TSharedPtr<FDetailTreeNode>& InDetailTreeNode)
		: DetailTreeNode(InDetailTreeNode)
	{ };

	/** Initializes the detail tree item, creating any child tree items needed */
	void Initialize(const FOnFilterDetailTreeNode& NodeFilter);

	/** Gets the parent detail tree item of this item */
	TWeakPtr<FColorGradingDetailTreeItem> GetParent() const { return Parent; }

	/** Gets whether this tree item has any children */
	bool HasChildren() const { return Children.Num() > 0; }

	/** Gets the list of child tree items of this item */
	void GetChildren(TArray<TSharedRef<FColorGradingDetailTreeItem>>& OutChildren) const;

	/** Gets the underlying IDetailTreeNode this detail tree item wraps */
	TWeakPtr<IDetailTreeNode> GetDetailTreeNode() const;

	/** Gets the property handle of the property this detail tree item represents */
	TSharedPtr<IPropertyHandle> GetPropertyHandle() const { return PropertyHandle; }

	/** Gets the name of this detail tree item */
	FName GetNodeName() const;

	/** Gets whether this detail tree item should be expanded */
	bool ShouldBeExpanded() const;

	/** Raised when this detail tree item's expansion state has been changed */
	void OnItemExpansionChanged(bool bIsExpanded, bool bShouldSaveState);

	/** Gets whether the "reset to default" button should be visible for this detail tree item */
	bool IsResetToDefaultVisible() const;

	/** Resets the property this detail tree item represents to its default value */
	void ResetToDefault();

	/** Gets an attribute that can be used to determine if property editing is enabled for this detail tree item */
	TAttribute<bool> IsPropertyEditingEnabled() const;

	/** Gets whether this detail tree item is a category */
	bool IsCategory() const;

	/** Gets whether this detail tree item is an item */
	bool IsItem() const;

	/** Gets whether this detail tree item can be reordered through a drag drop action */
	bool IsReorderable() const;

	/** Gets whether this detail tree item can be copied */
	bool IsCopyable() const;

	/** Generates the row widgets for this detail tree item */
	void GenerateDetailWidgetRow(FDetailWidgetRow& OutDetailWidgetRow) const;

private:
	/** A weak pointer to the detail tree node this item wraps */
	TWeakPtr<FDetailTreeNode> DetailTreeNode;

	/** The property handle for the property this item represents */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** A weak pointer to this item's parent */
	TWeakPtr<FColorGradingDetailTreeItem> Parent;

	/** A list of children of this item */
	TArray<TSharedRef<FColorGradingDetailTreeItem>> Children;
};