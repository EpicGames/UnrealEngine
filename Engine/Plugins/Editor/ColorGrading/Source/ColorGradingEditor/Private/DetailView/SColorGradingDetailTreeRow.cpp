// Copyright Epic Games, Inc. All Rights Reserved.ColorGradingDetailView

#include "SColorGradingDetailTreeRow.h"

#include "DetailView/SColorGradingDetailView.h"
#include "FColorGradingDetailTreeItem.h"

#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IDetailDragDropHandler.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Layout/WidgetPath.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

namespace UE::ColorGradingDetailTreeRow
{
	DECLARE_DELEGATE_RetVal_OneParam(FSlateColor, FOnGetIndentBackgroundColor, int32);

	// Copy of private SDetailRowIndent widget defined at PropertyEditor/Private/SDetailRowIndent.h
	class SDetailRowIndent : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SDetailRowIndent) {}
			SLATE_ATTRIBUTE(int32, Indent)
			SLATE_EVENT(FOnGetIndentBackgroundColor, OnGetIndentBackgroundColor)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs)
		{
			Indent = InArgs._Indent;
			OnGetIndentBackgroundColor = InArgs._OnGetIndentBackgroundColor;

			ChildSlot
			[
				SNew(SBox)
				.WidthOverride(this, &SDetailRowIndent::GetIndentWidth)
			];
		}


	private:
		virtual int32 OnPaint(const FPaintArgs& Args,
			const FGeometry& AllottedGeometry,
			const FSlateRect& MyCullingRect,
			FSlateWindowElementList& OutDrawElements,
			int32 LayerId,
			const FWidgetStyle& InWidgetStyle,
			bool bParentEnabled) const override
		{
			const FSlateBrush* BackgroundBrush = FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle");
			const FSlateBrush* DropShadowBrush = FAppStyle::Get().GetBrush("DetailsView.ArrayDropShadow");

			int32 IndentLevel = Indent.Get(0);
			for (int32 IndentIndex = 0; IndentIndex < IndentLevel; ++IndentIndex)
			{
				FSlateColor BackgroundColor = OnGetIndentBackgroundColor.IsBound() ? OnGetIndentBackgroundColor.Execute(IndentIndex) : FSlateColor(FLinearColor::White);

				FSlateDrawElement::MakeBox(
					OutDrawElements,
					LayerId,
					AllottedGeometry.ToPaintGeometry(FVector2D(16, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform(FVector2D(16 * IndentIndex, 0))),
					BackgroundBrush,
					ESlateDrawEffect::None,
					BackgroundColor.GetColor(InWidgetStyle)
				);

				FSlateDrawElement::MakeBox(
					OutDrawElements,
					LayerId + 1,
					AllottedGeometry.ToPaintGeometry(FVector2D(16, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform(FVector2D(16 * IndentIndex, 0))),
					DropShadowBrush
				);
			}

			return LayerId + 1;
		}

		FOptionalSize GetIndentWidth() const
		{
			return Indent.Get(0) * 16.0f;
		}

	private:
		TAttribute<int32> Indent;
		FOnGetIndentBackgroundColor OnGetIndentBackgroundColor;
	};

	// Copy of private SEditConditionWidget defined in PropertyEditor/Private/PropertyEditorHelpers.h
	class SEditConditionWidget : public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(SEditConditionWidget) {}
			SLATE_ATTRIBUTE(bool, EditConditionValue)
			SLATE_EVENT(FOnBooleanValueChanged, OnEditConditionValueChanged)
		SLATE_END_ARGS()

		void Construct(const FArguments& Args)
		{
			EditConditionValue = Args._EditConditionValue;
			OnEditConditionValueChanged = Args._OnEditConditionValueChanged;

			ChildSlot
			[
				SNew(SCheckBox)
				.OnCheckStateChanged(this, &SEditConditionWidget::OnEditConditionCheckChanged)
				.IsChecked(this, &SEditConditionWidget::OnGetEditConditionCheckState)
				.Visibility(this, &SEditConditionWidget::GetCheckBoxVisibility)
			];
		}

	private:
		bool HasEditConditionToggle() const
		{
			return OnEditConditionValueChanged.IsBound();
		}

		void OnEditConditionCheckChanged(ECheckBoxState CheckState)
		{
			checkSlow(HasEditConditionToggle());
			FScopedTransaction EditConditionChangedTransaction(NSLOCTEXT("PropertyEditor", "UpdatedEditConditionFmt", "Edit Condition Changed"));
			OnEditConditionValueChanged.ExecuteIfBound(CheckState == ECheckBoxState::Checked);
		}

		ECheckBoxState OnGetEditConditionCheckState() const
		{
			return EditConditionValue.Get() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		EVisibility GetCheckBoxVisibility() const
		{
			return HasEditConditionToggle() ? EVisibility::Visible : EVisibility::Collapsed;
		}

	private:
		TAttribute<bool> EditConditionValue;
		FOnBooleanValueChanged OnEditConditionValueChanged;
	};

	// Copy of private SConstrainedBox defined in PropertyEditor/Private/SConstrainedBox.h
	class SConstrainedBox : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SConstrainedBox) {}
			SLATE_DEFAULT_SLOT(FArguments, Content)
			SLATE_ATTRIBUTE(TOptional<float>, MinWidth)
			SLATE_ATTRIBUTE(TOptional<float>, MaxWidth)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs)
		{
			MinWidth = InArgs._MinWidth;
			MaxWidth = InArgs._MaxWidth;

			ChildSlot
			[
				InArgs._Content.Widget
			];
		}

		virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override
		{
			const float MinWidthVal = MinWidth.Get().Get(0.0f);
			const float MaxWidthVal = MaxWidth.Get().Get(0.0f);

			if (MinWidthVal == 0.0f && MaxWidthVal == 0.0f)
			{
				return SCompoundWidget::ComputeDesiredSize(LayoutScaleMultiplier);
			}
			else
			{
				FVector2D ChildSize = ChildSlot.GetWidget()->GetDesiredSize();

				float XVal = FMath::Max(MinWidthVal, ChildSize.X);
				if (MaxWidthVal > MinWidthVal)
				{
					XVal = FMath::Min(MaxWidthVal, XVal);
				}

				return FVector2D(XVal, ChildSize.Y);
			}
		}

	private:
		TAttribute< TOptional<float> > MinWidth;
		TAttribute< TOptional<float> > MaxWidth;
	};

	// Copy of private FArrayRowDragDropOp defined in PropertyEditor/Private/SDetailSingleItemRow.h
	class FArrayRowDragDropOp : public FDecoratedDragDropOp
	{
	public:
		DRAG_DROP_OPERATOR_TYPE(FArrayRowDragDropOp, FDecoratedDragDropOp)

		FArrayRowDragDropOp(const TSharedPtr<SColorGradingDetailTreeRow>& InRowBeingDragged)
			: RowBeingDragged(InRowBeingDragged)
		{
			MouseCursor = EMouseCursor::GrabHandClosed;
		}

		void Init()
		{
			SetValidTarget(false);
			SetupDefaults();
			Construct();
		}

		void SetValidTarget(bool IsValidTarget)
		{
			if (IsValidTarget)
			{
				CurrentHoverText = NSLOCTEXT("ArrayDragDrop", "PlaceRowHere", "Place Row Here");
				CurrentIconBrush = FAppStyle::GetBrush("Graph.ConnectorFeedback.OK");
			}
			else
			{
				CurrentHoverText = NSLOCTEXT("ArrayDragDrop", "CannotPlaceRowHere", "Cannot Place Row Here");
				CurrentIconBrush = FAppStyle::GetBrush("Graph.ConnectorFeedback.Error");
			}
		}

		TWeakPtr<SColorGradingDetailTreeRow> RowBeingDragged;
	};

	// Copy of private SArrayRowHandle defined in PropertyEditor/Private/SDetailSingleItemRow.h
	class SArrayRowHandle : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SArrayRowHandle) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedPtr<SColorGradingDetailTreeRow>& InParentRow)
		{
			ParentRow = InParentRow;
			SetCursor(EMouseCursor::GrabHand);

			ChildSlot
			[
				SNew(SBox)
				.Padding(0.0f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.WidthOverride(16.0f)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("VerticalBoxDragIndicatorShort"))
				]
			];
		}

		virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
		{
			return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
		};

		virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
		{
			if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
			{
				TSharedPtr<FDragDropOperation> DragDropOp = nullptr;
				if (ParentRow.IsValid())
				{
					DragDropOp = ParentRow.Pin()->CreateDragDropOperation();
				}

				if (DragDropOp.IsValid())
				{
					return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
				}
			}

			return FReply::Unhandled();
		}

	private:
		TWeakPtr<IDetailDragDropHandler> CustomDragDropHandler;
		TWeakPtr<SColorGradingDetailTreeRow> ParentRow;
	};
}

void SColorGradingDetailTreeRow::Construct(const FArguments& InArgs,
	const TSharedRef<FColorGradingDetailTreeItem>& InDetailTreeItem,
	const TSharedRef<STableViewBase>& InOwnerTableView,
	const FDetailColumnSizeData& InColumnSizeData)
{
	DetailTreeItem = InDetailTreeItem;

	TWeakPtr<STableViewBase> OwnerTableViewWeak = InOwnerTableView;

	ExpanderArrowWidget = SNew(SExpanderArrow, SharedThis(this))
		.BaseIndentLevel(0)
		.IndentAmount(0.0f);

	IndentWidget = SNew(UE::ColorGradingDetailTreeRow::SDetailRowIndent)
		.Indent(this, &SColorGradingDetailTreeRow::GetPropertyIndent)
		.OnGetIndentBackgroundColor(this, &SColorGradingDetailTreeRow::GetIndentBackgroundColor);

	STableRow::FArguments TableRowArgs = STableRow::FArguments()
		.Style(FAppStyle::Get(), "DetailsView.TreeView.TableRow")
		.ShowSelection(false)
		.OnDragLeave(this, &SColorGradingDetailTreeRow::OnRowDragLeave)
		.OnAcceptDrop(this, &SColorGradingDetailTreeRow::OnRowAcceptDrop)
		.OnCanAcceptDrop(this, &SColorGradingDetailTreeRow::CanRowAcceptDrop)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("DetailsView.GridLine"))
			.Padding(FMargin(0, 0, 0, 1))
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SNew(SBox)
				.MinDesiredHeight(26.0f)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					[
						SNew(SBorder)
						.Padding(0)
						.BorderImage(this, &SColorGradingDetailTreeRow::GetRowBackgroundBrush)
						.BorderBackgroundColor(this, &SColorGradingDetailTreeRow::GetRowBackgroundColor)
						[
							CreatePropertyWidget(InColumnSizeData)
						]
					]

					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.AutoWidth()
					[
						SNew(SBorder)
						.Padding(FMargin(0, 0, 16.0f, 0))
						.BorderImage(this, &SColorGradingDetailTreeRow::GetScrollWellBackgroundBrush, OwnerTableViewWeak)
						.BorderBackgroundColor(this, &SColorGradingDetailTreeRow::GetScrollWellBackgroundColor, OwnerTableViewWeak)
					]
				]
			]
		];

	STableRow<TSharedRef<FColorGradingDetailTreeItem>>::Construct(TableRowArgs, InOwnerTableView);
}

void SColorGradingDetailTreeRow::ConstructChildren(ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent)
{
	InnerContentSlot = nullptr;
	SetContent(InContent);
}

FReply SColorGradingDetailTreeRow::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (DetailTreeItem.IsValid() && MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && !StaticCastSharedRef<STableViewBase>(OwnerTablePtr.Pin()->AsWidget())->IsRightClickScrolling())
	{
		FMenuBuilder MenuBuilder(true, nullptr, nullptr, true);

		bool bShouldOpenMenu = false;

		if (DetailTreeItem.Pin()->HasChildren())
		{
			bShouldOpenMenu = true;

			FUIAction ExpandAllAction(FExecuteAction::CreateSP(this, &SColorGradingDetailTreeRow::SetExpansionStateForAll, true));
			FUIAction CollapseAllAction(FExecuteAction::CreateSP(this, &SColorGradingDetailTreeRow::SetExpansionStateForAll, false));

			MenuBuilder.BeginSection(NAME_None, NSLOCTEXT("PropertyView", "ExpansionHeading", "Expansion"));
			MenuBuilder.AddMenuEntry(NSLOCTEXT("PropertyView", "CollapseAll", "Collapse All"), NSLOCTEXT("PropertyView", "CollapseAll_ToolTip", "Collapses this item and all children"), FSlateIcon(), CollapseAllAction);
			MenuBuilder.AddMenuEntry(NSLOCTEXT("PropertyView", "ExpandAll", "Expand All"), NSLOCTEXT("PropertyView", "ExpandAll_ToolTip", "Expands this item and all children"), FSlateIcon(), ExpandAllAction);
			MenuBuilder.EndSection();
		}

		if (DetailTreeItem.Pin()->IsCopyable())
		{
			bShouldOpenMenu = true;

			// Hide separator line if it only contains the SearchWidget, making the next 2 elements the top of the list
			if (MenuBuilder.GetMultiBox()->GetBlocks().Num() > 1)
			{
				MenuBuilder.AddMenuSeparator();
			}

			FUIAction CopyAction = WidgetRow.IsCopyPasteBound() ?
				WidgetRow.CopyMenuAction :
				FExecuteAction::CreateSP(this, &SColorGradingDetailTreeRow::CopyPropertyValue);

			MenuBuilder.AddMenuEntry(
				NSLOCTEXT("PropertyView", "CopyProperty", "Copy"),
				NSLOCTEXT("PropertyView", "CopyProperty_ToolTip", "Copy this property value"),
				FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
				CopyAction);

			FUIAction PasteAction = WidgetRow.IsCopyPasteBound() ?
				WidgetRow.PasteMenuAction :
				FUIAction(
					FExecuteAction::CreateSP(this, &SColorGradingDetailTreeRow::PastePropertyValue),
					FCanExecuteAction::CreateSP(this, &SColorGradingDetailTreeRow::CanPastePropertyValue));

			MenuBuilder.AddMenuEntry(
				NSLOCTEXT("PropertyView", "PasteProperty", "Paste"),
				NSLOCTEXT("PropertyView", "PasteProperty_ToolTip", "Paste the copied value here"),
				FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Paste"),
				PasteAction);

			MenuBuilder.AddMenuEntry(
				NSLOCTEXT("PropertyView", "CopyPropertyDisplayName", "Copy Display Name"),
				NSLOCTEXT("PropertyView", "CopyPropertyDisplayName_ToolTip", "Copy the display name of this property to the system clipboard."),
				FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
				FExecuteAction::CreateSP(this, &SColorGradingDetailTreeRow::CopyPropertyName));
		}

		if (WidgetRow.CustomMenuItems.Num() > 0)
		{
			bShouldOpenMenu = true;

			// Hide separator line if it only contains the SearchWidget, making the next 2 elements the top of the list
			if (MenuBuilder.GetMultiBox()->GetBlocks().Num() > 1)
			{
				MenuBuilder.AddMenuSeparator();
			}

			for (const FDetailWidgetRow::FCustomMenuData& CustomMenuData : WidgetRow.CustomMenuItems)
			{
				// Add the menu entry
				MenuBuilder.AddMenuEntry(
					CustomMenuData.Name,
					CustomMenuData.Tooltip,
					CustomMenuData.SlateIcon,
					CustomMenuData.Action);
			}
		}

		if (bShouldOpenMenu)
		{
			FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();

			FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuBuilder.MakeWidget(), MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect::ContextMenu);

			return FReply::Handled();
		}
	}

	return STableRow<TSharedRef<FColorGradingDetailTreeItem, ESPMode::ThreadSafe>>::OnMouseButtonUp(MyGeometry, MouseEvent);
}

TSharedPtr<IPropertyHandle> SColorGradingDetailTreeRow::GetPropertyHandle() const
{
	TSharedPtr<IPropertyHandle> PropertyHandle = nullptr;

	if (DetailTreeItem.IsValid())
	{
		PropertyHandle = DetailTreeItem.Pin()->GetPropertyHandle();
	}

	return PropertyHandle;
}

TSharedPtr<FDragDropOperation> SColorGradingDetailTreeRow::CreateDragDropOperation()
{
	if (WidgetRow.CustomDragDropHandler)
	{
		TSharedPtr<FDragDropOperation> DragOp = WidgetRow.CustomDragDropHandler->CreateDragDropOperation();
		return DragOp;
	}
	else
	{
		TSharedPtr<UE::ColorGradingDetailTreeRow::FArrayRowDragDropOp> ArrayDragOp = MakeShareable(new UE::ColorGradingDetailTreeRow::FArrayRowDragDropOp(SharedThis(this)));
		ArrayDragOp->Init();
		return ArrayDragOp;
	}
}

TSharedRef<SWidget> SColorGradingDetailTreeRow::CreatePropertyWidget(const FDetailColumnSizeData& InColumnSizeData)
{
	if (DetailTreeItem.IsValid())
	{
		TSharedPtr<FColorGradingDetailTreeItem> PinnedDetailTreeItem = DetailTreeItem.Pin();
		PinnedDetailTreeItem->GenerateDetailWidgetRow(WidgetRow);

		if (!WidgetRow.HasAnyContent())
		{
			return SNullWidget::NullWidget;
		}

		TSharedRef<SSplitter> Splitter = SNew(SSplitter)
			.Style(FAppStyle::Get(), "DetailsView.Splitter")
			.PhysicalSplitterHandleSize(1.0f)
			.HitDetectionSplitterHandleSize(5.0f)
			.Orientation(Orient_Horizontal);

		const bool bHasMultipleColumns = WidgetRow.HasColumns();

		TSharedRef<SHorizontalBox> NameColumnBox = SNew(SHorizontalBox)
			.Clipping(EWidgetClipping::OnDemand);

		if (IndentWidget.IsValid())
		{
			NameColumnBox->AddSlot()
				.AutoWidth()
				.Padding(0)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Fill)
				[
					IndentWidget.ToSharedRef()
				];
		}

		bool bNeedsReorderHandle = WidgetRow.CustomDragDropHandler.IsValid() || DetailTreeItem.Pin()->IsReorderable();
		if (bNeedsReorderHandle)
		{
			NameColumnBox->AddSlot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(-4, 0, -10, 0)
				.AutoWidth()
				[
					SNew(UE::ColorGradingDetailTreeRow::SArrayRowHandle, SharedThis(this))
					.IsEnabled(this, &SColorGradingDetailTreeRow::IsRowEnabled)
					.Visibility_Lambda([this]() { return IsHovered() ? EVisibility::Visible : EVisibility::Hidden; })
				];
		}

		if (ExpanderArrowWidget.IsValid())
		{
			NameColumnBox->AddSlot()
				.AutoWidth()
				.Padding(2.f, 0.f, 0.f, 0.f)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Fill)
				[
					ExpanderArrowWidget.ToSharedRef()
				];
		}

		NameColumnBox->AddSlot()
			.AutoWidth()
			.Padding(2.f, 0.f, 0.f, 0.f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(UE::ColorGradingDetailTreeRow::SEditConditionWidget)
				.EditConditionValue(WidgetRow.EditConditionValue)
				.OnEditConditionValueChanged(WidgetRow.OnEditConditionValueChanged)
			];

		if (bHasMultipleColumns)
		{
			WidgetRow.NameWidget.Widget->SetEnabled(TAttribute<bool>::CreateSP(this, &SColorGradingDetailTreeRow::IsRowEnabled));

			NameColumnBox->AddSlot()
				.FillWidth(1.f)
				.Padding(2.f, 0.f, 0.f, 0.f)
				.HAlign(WidgetRow.NameWidget.HorizontalAlignment)
				.VAlign(WidgetRow.NameWidget.VerticalAlignment)
				[
					WidgetRow.NameWidget.Widget
				];
		}
		else
		{
			NameColumnBox->SetEnabled(TAttribute<bool>::CreateSP(this, &SColorGradingDetailTreeRow::IsRowEnabled));

			NameColumnBox->AddSlot()
				.FillWidth(1.f)
				.Padding(2.f, 0.f, 0.f, 0.f)
				.HAlign(WidgetRow.WholeRowWidget.HorizontalAlignment)
				.VAlign(WidgetRow.WholeRowWidget.VerticalAlignment)
				[
					WidgetRow.WholeRowWidget.Widget
				];
		}

		Splitter->AddSlot()
			.Value(InColumnSizeData.GetNameColumnWidth())
			.OnSlotResized(InColumnSizeData.GetOnNameColumnResized())
			[
				NameColumnBox
			];

		if (bHasMultipleColumns && WidgetRow.HasValueContent())
		{
			WidgetRow.ValueWidget.Widget->SetEnabled(TAttribute<bool>::CreateSP(this, &SColorGradingDetailTreeRow::IsRowValueEnabled));
			WidgetRow.ExtensionWidget.Widget->SetEnabled(TAttribute<bool>::CreateSP(this, &SColorGradingDetailTreeRow::IsRowEnabled));

			Splitter->AddSlot()
				.Value(InColumnSizeData.GetValueColumnWidth())
				.OnSlotResized(InColumnSizeData.GetOnValueColumnResized())
				[
					SNew(SHorizontalBox)
					.Clipping(EWidgetClipping::OnDemand)

					+ SHorizontalBox::Slot()
					.HAlign(WidgetRow.ValueWidget.HorizontalAlignment)
					.VAlign(WidgetRow.ValueWidget.VerticalAlignment)
					.Padding(12.0f, 0.0f, 2.0f, 0.0f)
					[
						SNew(UE::ColorGradingDetailTreeRow::SConstrainedBox)
						.MinWidth(WidgetRow.ValueWidget.MinWidth)
						.MaxWidth(WidgetRow.ValueWidget.MaxWidth)
						[
							WidgetRow.ValueWidget.Widget
						]
					]

					+ SHorizontalBox::Slot()
					.HAlign(WidgetRow.ExtensionWidget.HorizontalAlignment)
					.VAlign(WidgetRow.ExtensionWidget.VerticalAlignment)
					.Padding(5,0,0,0)
					.AutoWidth()
					[
						WidgetRow.ExtensionWidget.Widget
					]
				];
		}

		if (PinnedDetailTreeItem->IsItem())
		{
			TArray<FPropertyRowExtensionButton> ExtensionButtons;

			FPropertyRowExtensionButton& ResetToDefaultButton = ExtensionButtons.AddDefaulted_GetRef();
			ResetToDefaultButton.Label = NSLOCTEXT("PropertyEditor", "ResetToDefault", "Reset to Default");
			ResetToDefaultButton.UIAction = FUIAction(
				FExecuteAction::CreateSP(this, &SColorGradingDetailTreeRow::OnResetToDefaultClicked),
				FCanExecuteAction::CreateSP(this, &SColorGradingDetailTreeRow::CanResetToDefault)
			);

			// We could just collapse the Reset to Default button by setting the FIsActionButtonVisible delegate,
			// but this would cause the reset to defaults not to reserve space in the toolbar and not be aligned across all rows.
			// Instead, we show an empty icon and tooltip and disable the button.
			static FSlateIcon EnabledResetToDefaultIcon(FAppStyle::Get().GetStyleSetName(), "PropertyWindow.DiffersFromDefault");
			static FSlateIcon DisabledResetToDefaultIcon(FAppStyle::Get().GetStyleSetName(), "NoBrush");
			ResetToDefaultButton.Icon = TAttribute<FSlateIcon>::Create([this]()
			{
				if (DetailTreeItem.IsValid() && DetailTreeItem.Pin()->IsResetToDefaultVisible())
				{
					return EnabledResetToDefaultIcon;
				}

				return DisabledResetToDefaultIcon;
			});

			ResetToDefaultButton.ToolTip = TAttribute<FText>::Create([this]()
			{
				if (DetailTreeItem.IsValid() && DetailTreeItem.Pin()->IsResetToDefaultVisible())
				{
					return NSLOCTEXT("PropertyEditor", "ResetToDefaultPropertyValueToolTip", "Reset this property to its default value.");
				}

				return FText::GetEmpty();
			});

			CreateGlobalExtensionWidgets(ExtensionButtons);

			FSlimHorizontalToolBarBuilder ToolbarBuilder(TSharedPtr<FUICommandList>(), FMultiBoxCustomization::None);
			ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);
			ToolbarBuilder.SetStyle(&FAppStyle::Get(), "DetailsView.ExtensionToolBar");
			ToolbarBuilder.SetIsFocusable(false);

			for (const FPropertyRowExtensionButton& Extension : ExtensionButtons)
			{
				ToolbarBuilder.AddToolBarButton(Extension.UIAction, NAME_None, Extension.Label, Extension.ToolTip, Extension.Icon);
			}

			Splitter->AddSlot()
				.Value(InColumnSizeData.GetRightColumnWidth())
				.OnSlotResized(InColumnSizeData.GetOnRightColumnResized())
				.MinSize(InColumnSizeData.GetRightColumnMinWidth())
				[
					SNew(SBox)
					.Padding(0)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					[
						ToolbarBuilder.MakeWidget()
					]
				];
		}

		return Splitter;
	}

	return SNullWidget::NullWidget;
}

void SColorGradingDetailTreeRow::CreateGlobalExtensionWidgets(TArray<FPropertyRowExtensionButton>& OutExtensions) const
{
	// fetch global extension widgets 
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FOnGenerateGlobalRowExtensionArgs Args;
	Args.OwnerTreeNode = DetailTreeItem.Pin()->GetDetailTreeNode();
	Args.PropertyHandle = DetailTreeItem.Pin()->GetPropertyHandle();

	PropertyEditorModule.GetGlobalRowExtensionDelegate().Broadcast(Args, OutExtensions);
}

bool SColorGradingDetailTreeRow::IsRowEnabled() const
{
	if (DetailTreeItem.IsValid())
	{
		// Categories are always enabled
		if (DetailTreeItem.Pin()->IsCategory())
		{
			return true;
		}

		return DetailTreeItem.Pin()->IsPropertyEditingEnabled().Get(true) && 
			WidgetRow.IsEnabledAttr.Get(true) && 
			WidgetRow.EditConditionValue.Get(true);
	}

	return false;
}

bool SColorGradingDetailTreeRow::IsRowValueEnabled() const
{
	return IsRowEnabled() && WidgetRow.IsValueEnabledAttr.Get(true);
}

int32 SColorGradingDetailTreeRow::GetPropertyIndent() const
{
	const int32 PropertyIndentLevel = FMath::Max(GetIndentLevel() - 1, 0);
	return PropertyIndentLevel;
}

FSlateColor SColorGradingDetailTreeRow::GetIndentBackgroundColor(int32 Indent) const
{
	int32 ColorIndex = 0;
	int32 Increment = 1;

	for (int32 Index = 0; Index < Indent; ++Index)
	{
		ColorIndex += Increment;

		if (ColorIndex == 0 || ColorIndex == 3)
		{
			Increment = -Increment;
		}
	}

	static const uint8 ColorOffsets[] =
	{
		0, 4, (4 + 2), (6 + 4), (10 + 6)
	};

	const FSlateColor BaseSlateColor = IsHovered() ?
		FAppStyle::Get().GetSlateColor("Colors.Header") :
		FAppStyle::Get().GetSlateColor("Colors.Panel");

	const FColor BaseColor = BaseSlateColor.GetSpecifiedColor().ToFColor(true);

	const FColor ColorWithOffset(
		BaseColor.R + ColorOffsets[ColorIndex],
		BaseColor.G + ColorOffsets[ColorIndex],
		BaseColor.B + ColorOffsets[ColorIndex]);

	return FSlateColor(FLinearColor::FromSRGBColor(ColorWithOffset));
}

const FSlateBrush* SColorGradingDetailTreeRow::GetRowBackgroundBrush() const
{
	const bool bIsCategory = DetailTreeItem.IsValid() ? DetailTreeItem.Pin()->IsCategory() : false;
	return bIsCategory ? FAppStyle::Get().GetBrush("DetailsView.CategoryTop") : FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle");
}

FSlateColor SColorGradingDetailTreeRow::GetRowBackgroundColor() const
{
	bool bIsCategory = DetailTreeItem.IsValid() ? DetailTreeItem.Pin()->IsCategory() : false;
	return bIsCategory ? FSlateColor(FLinearColor::White) : GetIndentBackgroundColor(GetPropertyIndent());
}

const FSlateBrush* SColorGradingDetailTreeRow::GetScrollWellBackgroundBrush(TWeakPtr<STableViewBase> OwnerTableViewWeak) const
{
	const bool bIsScrollBarVisible = OwnerTableViewWeak.Pin()->GetScrollbarVisibility() == EVisibility::Visible;

	if (bIsScrollBarVisible)
	{
		return FAppStyle::Get().GetBrush("DetailsView.GridLine");
	}

	return GetRowBackgroundBrush();
}

FSlateColor SColorGradingDetailTreeRow::GetScrollWellBackgroundColor(TWeakPtr<STableViewBase> OwnerTableViewWeak) const
{
	const bool bIsScrollBarVisible = OwnerTableViewWeak.Pin()->GetScrollbarVisibility() == EVisibility::Visible;

	if (bIsScrollBarVisible)
	{
		return FSlateColor(EStyleColor::White);
	}

	return GetRowBackgroundColor();
}

void SColorGradingDetailTreeRow::OnResetToDefaultClicked()
{
	if (DetailTreeItem.IsValid())
	{
		if (WidgetRow.CustomResetToDefault.IsSet())
		{
			WidgetRow.CustomResetToDefault.GetValue().OnResetToDefaultClicked(DetailTreeItem.Pin()->GetPropertyHandle());
		}
		else
		{
			DetailTreeItem.Pin()->ResetToDefault();
		}
	}
}

bool SColorGradingDetailTreeRow::CanResetToDefault() const
{
	if (DetailTreeItem.IsValid())
	{
		if (WidgetRow.CustomResetToDefault.IsSet())
		{
			return WidgetRow.CustomResetToDefault.GetValue().IsResetToDefaultVisible(DetailTreeItem.Pin()->GetPropertyHandle());
		}

		return DetailTreeItem.Pin()->IsResetToDefaultVisible() && IsRowValueEnabled();
	}

	return false;
}

void SColorGradingDetailTreeRow::OnRowDragLeave(const FDragDropEvent& DragDropEvent)
{
	if (TSharedPtr<FDecoratedDragDropOp> DecoratedOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>())
	{
		DecoratedOp->ResetToDefaultToolTip();
	}
}

FReply SColorGradingDetailTreeRow::OnRowAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedRef<FColorGradingDetailTreeItem> TargetItem)
{
	if (WidgetRow.CustomDragDropHandler)
	{
		if (WidgetRow.CustomDragDropHandler->AcceptDrop(DragDropEvent, DropZone))
		{
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	if (TSharedPtr<UE::ColorGradingDetailTreeRow::FArrayRowDragDropOp> ArrayRowDropOp = DragDropEvent.GetOperationAs<UE::ColorGradingDetailTreeRow::FArrayRowDragDropOp>())
	{
		if (ArrayRowDropOp->RowBeingDragged.IsValid())
		{
			TSharedPtr<IPropertyHandle> SwappingPropertyHandle = ArrayRowDropOp->RowBeingDragged.Pin()->GetPropertyHandle();
			TSharedPtr<IPropertyHandle> ThisPropertyHandle = GetPropertyHandle();

			if (SwappingPropertyHandle.IsValid() && ThisPropertyHandle.IsValid())
			{
				const int32 OriginalIndex = SwappingPropertyHandle->GetIndexInArray();
				const int32 NewIndex = GetDropNewIndex(OriginalIndex, ThisPropertyHandle->GetIndexInArray(), DropZone);

				// Need to swap the moving and target expansion states before saving
				const bool bSwappingRowExpansion = ArrayRowDropOp->RowBeingDragged.Pin()->IsItemExpanded();
				const bool bThisRowExpansion = IsItemExpanded();

				if (bSwappingRowExpansion != bThisRowExpansion)
				{
					ArrayRowDropOp->RowBeingDragged.Pin()->ToggleExpansion();
					ToggleExpansion();
				}

				FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MoveRow", "Move Row"));

				SwappingPropertyHandle->GetParentHandle()->NotifyPreChange();
				SwappingPropertyHandle->GetParentHandle()->AsArray()->MoveElementTo(OriginalIndex, NewIndex);
				SwappingPropertyHandle->GetParentHandle()->NotifyPostChange(EPropertyChangeType::ArrayMove);

				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

TOptional<EItemDropZone> SColorGradingDetailTreeRow::CanRowAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedRef<FColorGradingDetailTreeItem> Type)
{
	if (WidgetRow.CustomDragDropHandler)
	{
		// Disallow drop between expanded parent item and its first child
		if (DropZone == EItemDropZone::BelowItem && IsItemExpanded())
		{
			return TOptional<EItemDropZone>();
		}

		return WidgetRow.CustomDragDropHandler->CanAcceptDrop(DragDropEvent, DropZone);
	}

	if (TSharedPtr<UE::ColorGradingDetailTreeRow::FArrayRowDragDropOp> ArrayRowDropOp = DragDropEvent.GetOperationAs<UE::ColorGradingDetailTreeRow::FArrayRowDragDropOp>())
	{
		// Can't drop onto another array item, so recompute our own drop zone to ensure it's above or below
		const FGeometry& Geometry = GetTickSpaceGeometry();
		const float LocalPointerY = Geometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition()).Y;
		const EItemDropZone OverrideDropZone = LocalPointerY < Geometry.GetLocalSize().Y * 0.5f ? EItemDropZone::AboveItem : EItemDropZone::BelowItem;

		const bool bIsValidDrop = IsValidDrop(DragDropEvent, OverrideDropZone);

		ArrayRowDropOp->SetValidTarget(bIsValidDrop);

		if (!bIsValidDrop)
		{
			return TOptional<EItemDropZone>();
		}

		return OverrideDropZone;
	}

	return TOptional<EItemDropZone>();
}

bool SColorGradingDetailTreeRow::IsValidDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const
{
	if (DropZone == EItemDropZone::OntoItem)
	{
		return false;
	}

	if (DropZone == EItemDropZone::BelowItem && IsItemExpanded())
	{
		return false;
	}

	if (TSharedPtr<UE::ColorGradingDetailTreeRow::FArrayRowDragDropOp> ArrayRowDropOp = DragDropEvent.GetOperationAs<UE::ColorGradingDetailTreeRow::FArrayRowDragDropOp>())
	{
		if (DetailTreeItem.IsValid() && !DetailTreeItem.Pin()->IsReorderable())
		{
			return false;
		}

		if (ArrayRowDropOp->RowBeingDragged.IsValid())
		{
			TSharedPtr<IPropertyHandle> SwappingPropertyHandle = ArrayRowDropOp->RowBeingDragged.Pin()->GetPropertyHandle();
			TSharedPtr<IPropertyHandle> ThisPropertyHandle = GetPropertyHandle();
			if (ThisPropertyHandle.IsValid() && SwappingPropertyHandle.IsValid() && !SwappingPropertyHandle->IsSamePropertyNode(ThisPropertyHandle))
			{
				const int32 OriginalIndex = SwappingPropertyHandle->GetIndexInArray();
				const int32 NewIndex = GetDropNewIndex(OriginalIndex, ThisPropertyHandle->GetIndexInArray(), DropZone);

				if (OriginalIndex != NewIndex)
				{
					if (SwappingPropertyHandle->GetParentHandle()->AsArray() && SwappingPropertyHandle->GetParentHandle()->IsSamePropertyNode(ThisPropertyHandle->GetParentHandle()))
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

int32 SColorGradingDetailTreeRow::GetDropNewIndex(int32 OriginalIndex, int32 DropOntoIndex, EItemDropZone DropZone) const
{
	check(DropZone != EItemDropZone::OntoItem);

	int32 NewIndex = DropOntoIndex;
	if (DropZone == EItemDropZone::BelowItem)
	{
		// If the drop zone is below, then we actually move it to the next item's index
		NewIndex++;
	}
	if (OriginalIndex < NewIndex)
	{
		// If the item is moved down the list, then all the other elements below it are shifted up one
		NewIndex--;
	}

	return ensure(NewIndex >= 0) ? NewIndex : 0;
}

void SColorGradingDetailTreeRow::CopyPropertyName()
{
	if (DetailTreeItem.IsValid())
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = DetailTreeItem.Pin()->GetPropertyHandle();
		if (PropertyHandle.IsValid())
		{
			FPlatformApplicationMisc::ClipboardCopy(*PropertyHandle->GetPropertyDisplayName().ToString());
		}
	}
}

void SColorGradingDetailTreeRow::CopyPropertyValue()
{
	if (DetailTreeItem.IsValid())
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = DetailTreeItem.Pin()->GetPropertyHandle();
		if (PropertyHandle.IsValid())
		{
			FString Value;
			if (PropertyHandle->GetValueAsFormattedString(Value, PPF_Copy) == FPropertyAccess::Success)
			{
				FPlatformApplicationMisc::ClipboardCopy(*Value);
			}
		}
	}
}

bool SColorGradingDetailTreeRow::CanPastePropertyValue()
{
	FString ClipboardContent;
	if (DetailTreeItem.IsValid())
	{
		if (DetailTreeItem.Pin()->GetPropertyHandle()->IsEditConst())
		{
			return false;
		}

		FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);
	}

	return !ClipboardContent.IsEmpty();
}

void SColorGradingDetailTreeRow::PastePropertyValue()
{
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	if (!ClipboardContent.IsEmpty() && DetailTreeItem.IsValid())
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = DetailTreeItem.Pin()->GetPropertyHandle();
		if (PropertyHandle.IsValid())
		{
			PropertyHandle->SetValueFromFormattedString(ClipboardContent, EPropertyValueSetFlags::InstanceObjects);
		}
	}
}

void SColorGradingDetailTreeRow::SetExpansionStateForAll(bool bShouldBeExpanded)
{
	if (DetailTreeItem.IsValid())
	{
		SetExpansionStateRecursive(DetailTreeItem.Pin().ToSharedRef(), bShouldBeExpanded);
	}
}

void SColorGradingDetailTreeRow::SetExpansionStateRecursive(const TSharedRef<FColorGradingDetailTreeItem>& TreeItem, bool bShouldBeExpanded)
{
	if (OwnerTablePtr.IsValid())
	{
		OwnerTablePtr.Pin()->Private_SetItemExpansion(TreeItem, bShouldBeExpanded);

		TArray<TSharedRef<FColorGradingDetailTreeItem>> Children;
		TreeItem->GetChildren(Children);

		for (const TSharedRef<FColorGradingDetailTreeItem>& Child : Children)
		{
			SetExpansionStateRecursive(Child, bShouldBeExpanded);
		}
	}
}
