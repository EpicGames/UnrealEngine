// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "DetailWidgetRow.h"

class FDetailColumnSizeData;
class FColorGradingDetailTreeItem;
class IPropertyHandle;

/** A copy/combination of SDetailSingleItemRow and SDetailCategoryTableRow defined in PropertyEditor/Private, used to display properties in a custom detail view widget */
class SColorGradingDetailTreeRow : public STableRow<TSharedRef<FColorGradingDetailTreeItem>>
{
public:
	SLATE_BEGIN_ARGS(SColorGradingDetailTreeRow) {}
	SLATE_END_ARGS()

	//~ STableRow Interface
	void Construct(const FArguments& InArgs, const TSharedRef<FColorGradingDetailTreeItem>& InDetailTreeItem, const TSharedRef<STableViewBase>& InOwnerTableView, const FDetailColumnSizeData& InColumnSizeData);
	virtual void ConstructChildren(ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	//~ End STableRow Interface

	/** Gets the property handle of the property this row represents, if applicable */
	TSharedPtr<IPropertyHandle> GetPropertyHandle() const;

	/** Creates a new drag drop operation for the property row */
	TSharedPtr<FDragDropOperation> CreateDragDropOperation();

private:
	/** Creates the widgets necessary to display the property this row represents */
	TSharedRef<SWidget> CreatePropertyWidget(const FDetailColumnSizeData& InColumnSizeData);

	/** Broadcasts the property editor module's global row extension delegate */
	void CreateGlobalExtensionWidgets(TArray<FPropertyRowExtensionButton>& OutExtensions) const;

	/** Gets whether the row is enabled */
	bool IsRowEnabled() const;

	/** Gets whether the row's value is enabled */
	bool IsRowValueEnabled() const;

	/** Gets the level of indent this row is at */
	int32 GetPropertyIndent() const;

	/** Gets the background color for the row depending on its level of indent */
	FSlateColor GetIndentBackgroundColor(int32 Indent) const;

	/** Gets the brush for the row's background border */
	const FSlateBrush* GetRowBackgroundBrush() const;

	/** Gets the color for the row's background border */
	FSlateColor GetRowBackgroundColor() const;

	/** Gets the brush for the row's scroll well */
	const FSlateBrush* GetScrollWellBackgroundBrush(TWeakPtr<STableViewBase> OwnerTableViewWeak) const;

	/** Gets the color for the row's scroll well */
	FSlateColor GetScrollWellBackgroundColor(TWeakPtr<STableViewBase> OwnerTableViewWeak) const;

	/** Raised when the property's Reset to Default button is pressed */
	void OnResetToDefaultClicked();

	/** Gets whether the property's Reset to Default button can be pressed */
	bool CanResetToDefault() const;

	/**  Raised when a drag drop event leaves this row */
	void OnRowDragLeave(const FDragDropEvent& DragDropEvent);

	/** Raised when a drag drop event is released on this row */
	FReply OnRowAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedRef<FColorGradingDetailTreeItem> TargetItem);

	/** Indicates whether the drag drop event can be accepted by this row */
	TOptional<EItemDropZone> CanRowAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedRef<FColorGradingDetailTreeItem> Type);

	/** Gets whether the specified drag drop event is allowed to be dropped onto this row */
	bool IsValidDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const;

	/** Gets the index to use when another row has been dragged onto this row, and an array element swap is taking place */
	int32 GetDropNewIndex(int32 OriginalIndex, int32 DropOntoIndex, EItemDropZone DropZone) const;

	/** Copies the property name to the clipboard */
	void CopyPropertyName();

	/** Copies the property value to the clipboard */
	void CopyPropertyValue();

	/** Checks to see if this property can receive a pasted value from the clipboard */
	bool CanPastePropertyValue();

	/** Pastes the value in the clipboard into this property */
	void PastePropertyValue();

	/** Sets the expansion state for this row and all of its children */
	void SetExpansionStateForAll(bool bShouldBeExpanded);

	/** Recursively sets the expansion state for the specified tree item and its children */
	void SetExpansionStateRecursive(const TSharedRef<FColorGradingDetailTreeItem>& TreeItem, bool bShouldBeExpanded);

private:
	/** A weak pointer to the detail tree item this row represents */
	TWeakPtr<FColorGradingDetailTreeItem> DetailTreeItem;

	/** The cached property row widgets that were created for this row */
	FDetailWidgetRow WidgetRow;

	/** The indent widget used to properly indent the row's widgets */
	TSharedPtr<SWidget> IndentWidget;
};