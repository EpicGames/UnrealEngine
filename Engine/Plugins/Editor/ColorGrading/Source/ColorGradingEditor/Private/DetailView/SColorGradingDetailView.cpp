// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailView/SColorGradingDetailView.h"

#include "FColorGradingDetailTreeItem.h"
#include "SColorGradingDetailTreeRow.h"

#include "DetailTreeNode.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

void SColorGradingDetailView::Construct(const FArguments& InArgs)
{
	PropertyRowGeneratorSource = InArgs._PropertyRowGeneratorSource;
	OnFilterDetailTreeNode = InArgs._OnFilterDetailTreeNode;

	ColumnSizeData.SetValueColumnWidth(0.5f);
	ColumnSizeData.SetRightColumnMinWidth(22);

	UpdateTreeNodes();

	TSharedRef<SScrollBar> ExternalScrollbar = SNew(SScrollBar);
	ExternalScrollbar->SetVisibility(TAttribute<EVisibility>(this, &SColorGradingDetailView::GetScrollBarVisibility));

	TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);

	VerticalBox->AddSlot()
	.FillHeight(1)
	.Padding(0)
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SAssignNew(DetailTree, SDetailTree)
			.TreeItemsSource(&RootTreeNodes)
			.OnGenerateRow(this, &SColorGradingDetailView::GenerateNodeRow)
			.OnGetChildren(this, &SColorGradingDetailView::GetChildrenForNode)
			.OnSetExpansionRecursive(this, &SColorGradingDetailView::OnSetExpansionRecursive)
			.OnRowReleased(this, &SColorGradingDetailView::OnRowReleased)
			.OnExpansionChanged(this, &SColorGradingDetailView::OnExpansionChanged)
			.SelectionMode(ESelectionMode::None)
			.HandleDirectionalNavigation(false)
			.AllowOverscroll(EAllowOverscroll::Yes)
			.ExternalScrollbar(ExternalScrollbar)
		]

		+ SOverlay::Slot()
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.WidthOverride(16.0f)
			[
				ExternalScrollbar
			]
		]
	];

	ChildSlot
	[
		VerticalBox
	];
}

void SColorGradingDetailView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (TreeItemsToSetExpansionState.Num() > 0)
	{
		for (const TPair<TWeakPtr<FColorGradingDetailTreeItem>, bool>& Pair : TreeItemsToSetExpansionState)
		{
			if (TSharedPtr<FColorGradingDetailTreeItem> DetailTreeItem = Pair.Key.Pin())
			{
				DetailTree->SetItemExpansion(DetailTreeItem.ToSharedRef(), Pair.Value);
			}
		}

		TreeItemsToSetExpansionState.Empty();
	}
}

void SColorGradingDetailView::Refresh()
{
	UpdateTreeNodes();
	DetailTree->RebuildList();
}

void SColorGradingDetailView::SaveExpandedItems()
{
	TSet<FString> ObjectTypes;
	if (PropertyRowGeneratorSource.IsValid())
	{
		const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = PropertyRowGeneratorSource->GetSelectedObjects();

		for (const TWeakObjectPtr<UObject>& Object : SelectedObjects)
		{
			if (Object.IsValid())
			{
				ObjectTypes.Add(Object->GetClass()->GetName());
			}
		}
	}

	FString ExpandedDetailNodesString;
	for (const FString& DetailNode : ExpandedDetailNodes)
	{
		ExpandedDetailNodesString += DetailNode;
		ExpandedDetailNodesString += TEXT(",");
	}

	for (const FString& ObjectType : ObjectTypes)
	{
		if (!ExpandedDetailNodesString.IsEmpty())
		{
			GConfig->SetString(TEXT("ColorGradingDetailsExpansion"), *ObjectType, *ExpandedDetailNodesString, GEditorPerProjectIni);
		}
		else
		{
			// If the expanded nodes string is empty but the saved expanded state is not, we want to save the empty string
			FString SavedExpandedDetailNodesString;
			GConfig->GetString(TEXT("ColorGradingDetailsExpansion"), *ObjectType, SavedExpandedDetailNodesString, GEditorPerProjectIni);

			if (!SavedExpandedDetailNodesString.IsEmpty())
			{
				GConfig->SetString(TEXT("ColorGradingDetailsExpansion"), *ObjectType, *ExpandedDetailNodesString, GEditorPerProjectIni);
			}
		}
	}
}

void SColorGradingDetailView::RestoreExpandedItems()
{
	TSet<FString> ObjectTypes;
	if (PropertyRowGeneratorSource.IsValid())
	{
		const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = PropertyRowGeneratorSource->GetSelectedObjects();

		for (const TWeakObjectPtr<UObject>& Object : SelectedObjects)
		{
			if (Object.IsValid())
			{
				ObjectTypes.Add(Object->GetClass()->GetName());
			}
		}
	}

	for (const FString& ObjectType : ObjectTypes)
	{
		FString SavedExpandedDetailNodesString;
		GConfig->GetString(TEXT("ColorGradingDetailsExpansion"), *ObjectType, SavedExpandedDetailNodesString, GEditorPerProjectIni);
		TArray<FString> SavedExpandedDetailNodes;
		SavedExpandedDetailNodesString.ParseIntoArray(SavedExpandedDetailNodes, TEXT(","), true);

		ExpandedDetailNodes.Append(SavedExpandedDetailNodes);
	}
}

void SColorGradingDetailView::UpdateTreeNodes()
{
	RootTreeNodes.Empty();

	RestoreExpandedItems();

	if (PropertyRowGeneratorSource.IsValid())
	{
		TArray<TSharedRef<IDetailTreeNode>> RawRootTreeNodes = PropertyRowGeneratorSource->GetRootTreeNodes();

		for (const TSharedRef<IDetailTreeNode>& RootTreeNode : RawRootTreeNodes)
		{
			bool bShouldDisplayNode = true;
			if (OnFilterDetailTreeNode.IsBound())
			{
				bShouldDisplayNode = OnFilterDetailTreeNode.Execute(RootTreeNode);
			}

			if (bShouldDisplayNode)
			{
				TSharedRef<FDetailTreeNode> CastRootTreeNode = StaticCastSharedRef<FDetailTreeNode>(RootTreeNode);
				TSharedRef<FColorGradingDetailTreeItem> RootTreeItem = MakeShared<FColorGradingDetailTreeItem>(CastRootTreeNode);
				RootTreeItem->Initialize(OnFilterDetailTreeNode);

				RootTreeNodes.Add(RootTreeItem);

				UpdateExpansionState(RootTreeItem);
			}
		}
	}
}

void SColorGradingDetailView::UpdateExpansionState(const TSharedRef<FColorGradingDetailTreeItem> InTreeItem)
{
	if (InTreeItem->IsCategory())
	{
		TreeItemsToSetExpansionState.Add(InTreeItem, InTreeItem->ShouldBeExpanded());
	}
	else if (InTreeItem->IsItem())
	{
		TSharedPtr<FColorGradingDetailTreeItem> ParentCategory = InTreeItem->GetParent().Pin();
		while (ParentCategory.IsValid() && !ParentCategory->IsCategory())
		{
			ParentCategory = ParentCategory->GetParent().Pin();
		}

		FString Key;
		if (ParentCategory.IsValid())
		{
			Key = ParentCategory->GetNodeName().ToString() + TEXT(".") + InTreeItem->GetNodeName().ToString();
		}
		else
		{
			Key = InTreeItem->GetNodeName().ToString();
		}

		const bool bShouldItemBeExpanded = ExpandedDetailNodes.Contains(Key) && InTreeItem->HasChildren();
		TreeItemsToSetExpansionState.Add(InTreeItem, bShouldItemBeExpanded);
	}

	TArray<TSharedRef<FColorGradingDetailTreeItem>> Children;
	InTreeItem->GetChildren(Children);

	for (const TSharedRef<FColorGradingDetailTreeItem>& Child : Children)
	{
		UpdateExpansionState(Child);
	}
}

TSharedRef<ITableRow> SColorGradingDetailView::GenerateNodeRow(TSharedRef<FColorGradingDetailTreeItem> InTreeItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SColorGradingDetailTreeRow, InTreeItem, OwnerTable, ColumnSizeData);
}

void SColorGradingDetailView::GetChildrenForNode(TSharedRef<FColorGradingDetailTreeItem> InTreeItem, TArray<TSharedRef<FColorGradingDetailTreeItem>>& OutChildren)
{
	InTreeItem->GetChildren(OutChildren);
}

void SColorGradingDetailView::SetNodeExpansionState(TSharedRef<FColorGradingDetailTreeItem> InTreeItem, bool bIsItemExpanded, bool bRecursive)
{
	TArray<TSharedRef<FColorGradingDetailTreeItem>> Children;
	InTreeItem->GetChildren(Children);

	if (Children.Num())
	{
		const bool bShouldSaveState = true;
		InTreeItem->OnItemExpansionChanged(bIsItemExpanded, bShouldSaveState);

		// Category nodes will save themselves to the editor config, but the item nodes can't, so manually save their expansion state here
		if (InTreeItem->IsItem())
		{
			TSharedPtr<FColorGradingDetailTreeItem> ParentCategory = InTreeItem->GetParent().Pin();
			while (ParentCategory.IsValid() && !ParentCategory->IsCategory())
			{
				ParentCategory = ParentCategory->GetParent().Pin();
			}

			FString Key;
			if (ParentCategory.IsValid())
			{
				Key = ParentCategory->GetNodeName().ToString() + TEXT(".") + InTreeItem->GetNodeName().ToString();
			}
			else
			{
				Key = InTreeItem->GetNodeName().ToString();
			}
			
			if (bIsItemExpanded)
			{
				ExpandedDetailNodes.Add(Key);
			}
			else
			{
				ExpandedDetailNodes.Remove(Key);
			}
		}

		if (bRecursive)
		{
			for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
			{
				SetNodeExpansionState(Children[ChildIndex], bIsItemExpanded, bRecursive);
			}
		}
	}
}

void SColorGradingDetailView::OnSetExpansionRecursive(TSharedRef<FColorGradingDetailTreeItem> InTreeNode, bool bIsItemExpanded)
{
	SetNodeExpansionState(InTreeNode, bIsItemExpanded, true);
	SaveExpandedItems();
}

void SColorGradingDetailView::OnExpansionChanged(TSharedRef<FColorGradingDetailTreeItem> InTreeNode, bool bIsItemExpanded)
{
	SetNodeExpansionState(InTreeNode, bIsItemExpanded, false);
	SaveExpandedItems();
}

void SColorGradingDetailView::OnRowReleased(const TSharedRef<ITableRow>& TableRow)
{
	// search upwards from the current keyboard-focused widget to see if it's contained in our row
	TSharedPtr<SWidget> CurrentWidget = FSlateApplication::Get().GetKeyboardFocusedWidget();
	while (CurrentWidget.IsValid())
	{
		if (CurrentWidget == TableRow->AsWidget())
		{
			// if so, clear focus so that any pending value changes are committed
			FSlateApplication::Get().ClearKeyboardFocus();
			return;
		}

		CurrentWidget = CurrentWidget->GetParentWidget();
	}
}

EVisibility SColorGradingDetailView::GetScrollBarVisibility() const
{
	const bool bShowScrollBar = RootTreeNodes.Num() > 0;
	return bShowScrollBar ? EVisibility::Visible : EVisibility::Collapsed;
}