// Copyright Epic Games, Inc. All Rights Reserved.

#include "SColorGradingColorWheel.h"

#include "Customizations/MathStructCustomizations.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Util/ColorGradingUtil.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/ColorGrading/SColorGradingComponentViewer.h"
#include "Widgets/Colors/SComplexGradient.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "ColorGradingEditor"

SColorGradingColorWheel::SColorGradingColorWheel()
{
	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}

	FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &SColorGradingColorWheel::OnPropertyValueChanged);
}

SColorGradingColorWheel::~SColorGradingColorWheel()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}

	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
}

void SColorGradingColorWheel::Construct(const FArguments& InArgs)
{
	ColorDisplayMode = InArgs._ColorDisplayMode;

	ColorPickerBox = SNew(SBox);
	
	ColorSlidersBox = SNew(SBox)
		.HAlign(HAlign_Fill)
		.MaxDesiredWidth(400.f)
		.MinDesiredWidth(400.f);

	HeaderBox = SNew(SBox);

	ChildSlot
	[
		SNew(SBox)
		.Padding(ColumnPadding.X, ColumnPadding.Y)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			[
				HeaderBox.ToSharedRef()
			]

			+ SVerticalBox::Slot()
			.FillHeight(1.0)
			.HAlign(HAlign_Fill)
			[
				SNew(SOverlay)

				// Standard/"short" layout
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SVerticalBox)
					.Visibility(this, &SColorGradingColorWheel::GetShortLayoutVisibility)

					+ SVerticalBox::Slot()
					.FillHeight(1.f)
					[
						ColorPickerBox.ToSharedRef()
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBox)
						.Visibility(this, &SColorGradingColorWheel::GetSlidersVisibility)
						[
							ColorSlidersBox.ToSharedRef()
						]
					]
				]

				// Tall layout
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SNew(SVerticalBox)
					.Visibility(this, &SColorGradingColorWheel::GetTallLayoutVisibility)

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						ColorPickerBox.ToSharedRef()
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						ColorSlidersBox.ToSharedRef()
					]
				]
			]
		]
	];

	if (InArgs._HeaderContent.IsValid())
	{
		HeaderBox->SetContent(InArgs._HeaderContent.ToSharedRef());
	}
}

void SColorGradingColorWheel::SetColorPropertyHandle(TSharedPtr<IPropertyHandle> InColorPropertyHandle)
{
	ColorGradingPicker.Reset();
	ColorPropertyMetadata.Reset();
	ComponentSliderDynamicMinValue.Reset();
	ComponentSliderDynamicMaxValue.Reset();

	ColorPropertyHandle = FTrackedVector4PropertyHandle(InColorPropertyHandle);

	if (ColorPropertyHandle.IsValidHandle())
	{
		ColorPropertyMetadata = GetColorPropertyMetadata();
		ComponentSliderDynamicMinValue = ColorPropertyMetadata->MinValue;
		ComponentSliderDynamicMaxValue = ColorPropertyMetadata->MaxValue;
	}

	RecalculateHSVColor();

	// Since many of the color picker slate properties are not attributes, we need to recreate the widget every time the property handle changes
	// to ensure the picker is configured correctly for the color property
	if (ColorPickerBox.IsValid())
	{
		ColorPickerBox->SetContent(CreateColorGradingPicker());
	}

	if (ColorSlidersBox.IsValid())
	{
		ColorSlidersBox->SetContent(CreateColorComponentSliders());
	}
}

void SColorGradingColorWheel::SetHeaderContent(const TSharedRef<SWidget>& HeaderContent)
{
	if (HeaderBox.IsValid())
	{
		HeaderBox->SetContent(HeaderContent);
	}
}

void SColorGradingColorWheel::PostUndo(bool bSuccess)
{
	RecalculateHSVColor();
}

void SColorGradingColorWheel::PostRedo(bool bSuccess)
{
	RecalculateHSVColor();
}

void SColorGradingColorWheel::OnPropertyValueChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (ColorPropertyHandle.IsSettingValue())
	{
		// If setting our own value, it's already handled (or will be)
		return;
	}

	if (TSharedPtr<const IPropertyHandle> PinnedPropertyHandle = ColorPropertyHandle.GetHandle())
	{
		uint32 NumChildren;
		if (PinnedPropertyHandle->GetNumChildren(NumChildren) == FPropertyAccess::Result::Success)
		{
			for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
			{
				if (PinnedPropertyHandle->GetChildHandle(ChildIndex)->GetProperty() == PropertyChangedEvent.Property)
				{
					RecalculateHSVColor();
					break;
				}
			}
		}
	}
}

TSharedRef<SWidget> SColorGradingColorWheel::SColorGradingColorWheel::CreateColorGradingPicker()
{
	if (ColorPropertyHandle.IsValidHandle())
	{
		return SNew(UE::ColorGrading::SColorGradingPicker)
			.DesiredWheelSize(this, &SColorGradingColorWheel::GetMaxWheelWidth)
			.ValueMin(GetMetadataMinValue())
			.ValueMax(GetMetadataMaxValue())
			.SliderValueMin(GetMetadataSliderMinValue())
			.SliderValueMax(GetMetadataSliderMaxValue())
			.MainDelta(GetMetadataDelta())
			.SupportDynamicSliderMinValue(GetMetadataSupportDynamicSliderMinValue())
			.SupportDynamicSliderMaxValue(GetMetadataSupportDynamicSliderMaxValue())
			.MainShiftMultiplier(GetMetadataShiftMultiplier())
			.MainCtrlMultiplier(GetMetadataCtrlMultiplier())
			.ColorGradingModes(ColorPropertyMetadata->ColorGradingMode)
			.OnColorCommitted(this, &SColorGradingColorWheel::CommitColor)
			.OnQueryCurrentColor(this, &SColorGradingColorWheel::GetColor)
			.AllowSpin(ColorPropertyHandle.GetHandle()->GetNumOuterObjects() == 1) // TODO: May want to find a way to support multiple objects
			.OnBeginSliderMovement(this, &SColorGradingColorWheel::BeginUsingColorPickerSlider)
			.OnEndSliderMovement(this, &SColorGradingColorWheel::EndUsingColorPickerSlider)
			.OnBeginMouseCapture(this, &SColorGradingColorWheel::BeginUsingColorPickerSlider)
			.OnEndMouseCapture(this, &SColorGradingColorWheel::EndUsingColorPickerSlider)
			.IsEnabled(this, &SColorGradingColorWheel::IsPropertyEnabled);
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SColorGradingColorWheel::CreateColorComponentSliders()
{
	if (ColorPropertyHandle.IsValidHandle())
	{
		TSharedRef<SVerticalBox> ColorSlidersVerticalBox = SNew(SVerticalBox);

		uint32 NumComponents = 4;
		for (uint32 ComponentIndex = 0; ComponentIndex < NumComponents; ++ComponentIndex)
		{
			TAttribute<UE::ColorGrading::EColorGradingComponent> ComponentGetter = TAttribute<UE::ColorGrading::EColorGradingComponent>::Create(
				TAttribute<UE::ColorGrading::EColorGradingComponent>::FGetter::CreateSP(this, &SColorGradingColorWheel::GetComponent, ComponentIndex)
			);

			ColorSlidersVerticalBox->AddSlot()
			.Padding(FMargin(0.0f, 4.0f, 0.0f, 0.0f))
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SNew(UE::ColorGrading::SColorGradingComponentViewer)
				.Component(ComponentGetter)
				.ColorGradingMode(ColorPropertyMetadata->ColorGradingMode)
				.Value(this, &SColorGradingColorWheel::GetComponentValue, ComponentIndex)
				.OnValueChanged(this, &SColorGradingColorWheel::SetComponentValue, ComponentIndex)
				.OnBeginSliderMovement(this, &SColorGradingColorWheel::BeginUsingComponentSlider, ComponentIndex)
				.OnEndSliderMovement(this, &SColorGradingColorWheel::EndUsingComponentSlider, ComponentIndex)
				.OnQueryCurrentColor(this, &SColorGradingColorWheel::GetColor)
				.ShiftMultiplier(ColorPropertyMetadata->ShiftMultiplier)
				.CtrlMultiplier(ColorPropertyMetadata->CtrlMultiplier)
				.SupportDynamicSliderMinValue(this, &SColorGradingColorWheel::ComponentSupportsDynamicSliderValue, ColorPropertyMetadata->bSupportDynamicSliderMinValue, ComponentIndex)
				.SupportDynamicSliderMaxValue(this, &SColorGradingColorWheel::ComponentSupportsDynamicSliderValue, ColorPropertyMetadata->bSupportDynamicSliderMaxValue, ComponentIndex)
				.OnDynamicSliderMinValueChanged(this, &SColorGradingColorWheel::UpdateComponentDynamicSliderMinValue)
				.OnDynamicSliderMaxValueChanged(this, &SColorGradingColorWheel::UpdateComponentDynamicSliderMaxValue)
				.MinValue(ColorPropertyMetadata->MinValue)
				.MaxValue(this, &SColorGradingColorWheel::GetComponentMaxValue, ColorPropertyMetadata->MaxValue, ComponentIndex)
				.MinSliderValue(this, &SColorGradingColorWheel::GetComponentMinSliderValue, ColorPropertyMetadata->SliderMinValue, ComponentIndex)
				.MaxSliderValue(this, &SColorGradingColorWheel::GetComponentMaxSliderValue, ColorPropertyMetadata->SliderMaxValue, ComponentIndex)
				.SliderExponent(ColorPropertyMetadata->SliderExponent)
				.SliderExponentNeutralValue(ColorPropertyMetadata->SliderMinValue.GetValue() + (ColorPropertyMetadata->SliderMaxValue.GetValue() - ColorPropertyMetadata->SliderMinValue.GetValue()) / 2.0f)
				.Delta(this, &SColorGradingColorWheel::GetComponentSliderDeltaValue, ColorPropertyMetadata->Delta, ComponentIndex)
				.IsEnabled(this, &SColorGradingColorWheel::IsPropertyEnabled)
			];
		}

		return ColorSlidersVerticalBox;
	}

	return SNullWidget::NullWidget;
}

SColorGradingColorWheel::FColorPropertyMetadata SColorGradingColorWheel::GetColorPropertyMetadata() const
{
	FColorPropertyMetadata PropertyMetadata;

	if (ColorPropertyHandle.IsValidHandle())
	{
		FProperty* Property = ColorPropertyHandle.GetHandle()->GetProperty();
		const FString ColorGradingModeString = Property->GetMetaData(TEXT("ColorGradingMode")).ToLower();

		if (!ColorGradingModeString.IsEmpty())
		{
			if (ColorGradingModeString.Compare(TEXT("saturation")) == 0)
			{
				PropertyMetadata.ColorGradingMode = UE::ColorGrading::EColorGradingModes::Saturation;
			}
			else if (ColorGradingModeString.Compare(TEXT("contrast")) == 0)
			{
				PropertyMetadata.ColorGradingMode = UE::ColorGrading::EColorGradingModes::Contrast;
			}
			else if (ColorGradingModeString.Compare(TEXT("gamma")) == 0)
			{
				PropertyMetadata.ColorGradingMode = UE::ColorGrading::EColorGradingModes::Gamma;
			}
			else if (ColorGradingModeString.Compare(TEXT("gain")) == 0)
			{
				PropertyMetadata.ColorGradingMode = UE::ColorGrading::EColorGradingModes::Gain;
			}
			else if (ColorGradingModeString.Compare(TEXT("offset")) == 0)
			{
				PropertyMetadata.ColorGradingMode = UE::ColorGrading::EColorGradingModes::Offset;
			}
		}

		const FString& MetaUIMinString = Property->GetMetaData(TEXT("UIMin"));
		const FString& MetaUIMaxString = Property->GetMetaData(TEXT("UIMax"));
		const FString& SliderExponentString = Property->GetMetaData(TEXT("SliderExponent"));
		const FString& DeltaString = Property->GetMetaData(TEXT("Delta"));
		const FString& LinearDeltaSensitivityString = Property->GetMetaData(TEXT("LinearDeltaSensitivity"));
		const FString& ShiftMultiplierString = Property->GetMetaData(TEXT("ShiftMultiplier"));
		const FString& CtrlMultiplierString = Property->GetMetaData(TEXT("CtrlMultiplier"));
		const FString& SupportDynamicSliderMaxValueString = Property->GetMetaData(TEXT("SupportDynamicSliderMaxValue"));
		const FString& SupportDynamicSliderMinValueString = Property->GetMetaData(TEXT("SupportDynamicSliderMinValue"));
		const FString& ClampMinString = Property->GetMetaData(TEXT("ClampMin"));
		const FString& ClampMaxString = Property->GetMetaData(TEXT("ClampMax"));

		const FString& UIMinString = MetaUIMinString.Len() ? MetaUIMinString : ClampMinString;
		const FString& UIMaxString = MetaUIMaxString.Len() ? MetaUIMaxString : ClampMaxString;

		float ClampMin = TNumericLimits<float>::Lowest();
		float ClampMax = TNumericLimits<float>::Max();

		if (!ClampMinString.IsEmpty())
		{
			TTypeFromString<float>::FromString(ClampMin, *ClampMinString);
		}

		if (!ClampMaxString.IsEmpty())
		{
			TTypeFromString<float>::FromString(ClampMax, *ClampMaxString);
		}

		float UIMin = TNumericLimits<float>::Lowest();
		float UIMax = TNumericLimits<float>::Max();
		TTypeFromString<float>::FromString(UIMin, *UIMinString);
		TTypeFromString<float>::FromString(UIMax, *UIMaxString);

		const float ActualUIMin = FMath::Max(UIMin, ClampMin);
		const float ActualUIMax = FMath::Min(UIMax, ClampMax);

		PropertyMetadata.MinValue = ClampMinString.Len() ? ClampMin : TOptional<float>();
		PropertyMetadata.MaxValue = ClampMaxString.Len() ? ClampMax : TOptional<float>();
		PropertyMetadata.SliderMinValue = (UIMinString.Len()) ? ActualUIMin : TOptional<float>();
		PropertyMetadata.SliderMaxValue = (UIMaxString.Len()) ? ActualUIMax : TOptional<float>();

		if (SliderExponentString.Len())
		{
			TTypeFromString<float>::FromString(PropertyMetadata.SliderExponent, *SliderExponentString);
		}

		if (DeltaString.Len())
		{
			TTypeFromString<float>::FromString(PropertyMetadata.Delta, *DeltaString);
		}

		if (LinearDeltaSensitivityString.Len())
		{
			TTypeFromString<int32>::FromString(PropertyMetadata.LinearDeltaSensitivity, *LinearDeltaSensitivityString);
			PropertyMetadata.Delta = (PropertyMetadata.Delta == 0.0f) ? 1.0f : PropertyMetadata.Delta;
		}

		PropertyMetadata.ShiftMultiplier = 10.f;
		if (ShiftMultiplierString.Len())
		{
			TTypeFromString<float>::FromString(PropertyMetadata.ShiftMultiplier, *ShiftMultiplierString);
		}

		PropertyMetadata.CtrlMultiplier = 0.1f;
		if (CtrlMultiplierString.Len())
		{
			TTypeFromString<float>::FromString(PropertyMetadata.CtrlMultiplier, *CtrlMultiplierString);
		}

		PropertyMetadata.bSupportDynamicSliderMaxValue = SupportDynamicSliderMaxValueString.Len() > 0 && SupportDynamicSliderMaxValueString.ToBool();
		PropertyMetadata.bSupportDynamicSliderMinValue = SupportDynamicSliderMinValueString.Len() > 0 && SupportDynamicSliderMinValueString.ToBool();
	}

	return PropertyMetadata;
}

bool SColorGradingColorWheel::IsPropertyEnabled() const
{
	if (ColorPropertyHandle.IsValidHandle())
	{
		return ColorPropertyHandle.GetHandle()->IsEditable();
	}

	return false;
}

EVisibility SColorGradingColorWheel::GetSlidersVisibility() const
{
	return GetTickSpaceGeometry().GetLocalSize().Y >= 294
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

EVisibility SColorGradingColorWheel::GetShortLayoutVisibility() const
{
	return ShouldUseTallLayout()
		? EVisibility::Collapsed
		: EVisibility::Visible;
}

EVisibility SColorGradingColorWheel::GetTallLayoutVisibility() const
{
	return ShouldUseTallLayout()
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

int32 SColorGradingColorWheel::GetMaxWheelWidth() const
{
	const float InnerWidth = GetTickSpaceGeometry().GetLocalSize().X - ColumnPadding.X * 2;
	return FMath::FloorToInt32(FMath::Max(InnerWidth, 0.f));
}

bool SColorGradingColorWheel::ShouldUseTallLayout() const
{
	// Switch to tall layout when the column space not occupied by the wheel is above this size
	const float RemainingSpaceThrehold = 450;
	return GetTickSpaceGeometry().GetLocalSize().Y >= (RemainingSpaceThrehold + GetMaxWheelWidth());
}

bool SColorGradingColorWheel::GetColor(FVector4& OutCurrentColor)
{
	return ColorPropertyHandle.GetValue(OutCurrentColor) == FPropertyAccess::Success;
}

void SColorGradingColorWheel::CommitColor(FVector4& NewValue, bool bShouldCommitValueChanges)
{
	FScopedTransaction Transaction(LOCTEXT("ColorWheel_TransactionName", "Color Grading Main Value"), bShouldCommitValueChanges);

	if (ColorPropertyHandle.IsValidHandle())
	{
		// Always perform a purely interactive change. We do this because it won't invoke reconstruction, which may cause that only the first 
		// element gets updated due to its change causing a component reconstruction and the remaining vector element property handles updating 
		// the trashed component.
		ColorPropertyHandle.SetValue(NewValue, EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable);

		// If not purely interactive, set the value with default flags.
		if (bShouldCommitValueChanges || !bIsUsingColorPickerSlider)
		{
			ColorPropertyHandle.SetValue(NewValue, EPropertyValueSetFlags::DefaultFlags);
		}

		TransactColorValue();
	}

	CurrentHSVColor = FLinearColor(NewValue.X, NewValue.Y, NewValue.Z).LinearRGBToHSV();
}

void SColorGradingColorWheel::TransactColorValue()
{
	if (ColorPropertyHandle.IsValidHandle())
	{
		TArray<UObject*> OuterObjects;
		ColorPropertyHandle.GetHandle()->GetOuterObjects(OuterObjects);
		for (UObject* Object : OuterObjects)
		{
			if (!Object)
			{
				continue;
			}

			if (!Object->HasAnyFlags(RF_Transactional))
			{
				Object->SetFlags(RF_Transactional);
			}

			SaveToTransactionBuffer(Object, false);
			SnapshotTransactionBuffer(Object);
		}
	}
}

void SColorGradingColorWheel::RecalculateHSVColor()
{
	if (ColorPropertyHandle.IsValidHandle())
	{
		FVector4 VectorValue;
		if (ColorPropertyHandle.GetValue(VectorValue) == FPropertyAccess::Success)
		{
			CurrentHSVColor = FLinearColor(VectorValue).LinearRGBToHSV();
		}
	}
}

void SColorGradingColorWheel::BeginUsingColorPickerSlider()
{
	bIsUsingColorPickerSlider = true;

	if (GEditor)
	{
		GEditor->BeginTransaction(LOCTEXT("ColorWheel_TransactionName", "Color Grading Main Value"));
	}
}

void SColorGradingColorWheel::EndUsingColorPickerSlider()
{
	bIsUsingColorPickerSlider = false;

	if (GEditor)
	{
		GEditor->EndTransaction();
	}
}

void SColorGradingColorWheel::BeginUsingComponentSlider(uint32 ComponentIndex)
{
	bIsUsingComponentSlider = true;

	if (GEditor)
	{
		GEditor->BeginTransaction(LOCTEXT("ColorWheel_TransactionName", "Color Grading Main Value"));
	}
}

void SColorGradingColorWheel::EndUsingComponentSlider(float NewValue, uint32 ComponentIndex)
{
	bIsUsingComponentSlider = false;
	SetComponentValue(NewValue, ComponentIndex);

	if (GEditor)
	{
		GEditor->EndTransaction();
	}
}

UE::ColorGrading::EColorGradingComponent SColorGradingColorWheel::GetComponent(uint32 ComponentIndex) const
{
	return UE::ColorGrading::GetColorGradingComponent(ColorDisplayMode.Get(), ComponentIndex);
}


TOptional<float> SColorGradingColorWheel::GetComponentValue(uint32 ComponentIndex) const
{
	if (ColorPropertyHandle.IsValidHandle())
	{
		FVector4 ColorValue;
		if (ColorPropertyHandle.GetValue(ColorValue) == FPropertyAccess::Success)
		{
			float Value = 0.0f;

			if (ColorDisplayMode.Get(UE::ColorGrading::EColorGradingColorDisplayMode::RGB) == UE::ColorGrading::EColorGradingColorDisplayMode::RGB)
			{
				Value = ColorValue[ComponentIndex];
			}
			else
			{
				Value = CurrentHSVColor.Component(ComponentIndex);
			}

			return Value;
		}
	}

	return TOptional<float>();
}

void SColorGradingColorWheel::SetComponentValue(float NewValue, uint32 ComponentIndex)
{
	if (ColorPropertyHandle.IsValidHandle())
	{
		FVector4 CurrentColorValue;
		if (ColorPropertyHandle.GetValue(CurrentColorValue) == FPropertyAccess::Success)
		{
			FVector4 NewColorValue = CurrentColorValue;

			if (ColorDisplayMode.Get(UE::ColorGrading::EColorGradingColorDisplayMode::RGB) == UE::ColorGrading::EColorGradingColorDisplayMode::RGB)
			{
				NewColorValue[ComponentIndex] = NewValue;

				if (ComponentIndex < 3)
				{
					CurrentHSVColor = FLinearColor(NewColorValue.X, NewColorValue.Y, NewColorValue.Z).LinearRGBToHSV();
				}
			}
			else
			{
				CurrentHSVColor.Component(ComponentIndex) = NewValue;
				NewColorValue = (FVector4)CurrentHSVColor.HSVToLinearRGB();
			}

			ColorPropertyHandle.SetValue(NewColorValue, bIsUsingComponentSlider ? EPropertyValueSetFlags::InteractiveChange : EPropertyValueSetFlags::DefaultFlags);
			TransactColorValue();
		}
	}
}

bool SColorGradingColorWheel::ComponentSupportsDynamicSliderValue(bool bDefaultValue, uint32 ComponentIndex) const
{
	if (bDefaultValue)
	{
		if (ColorDisplayMode.Get(UE::ColorGrading::EColorGradingColorDisplayMode::RGB) != UE::ColorGrading::EColorGradingColorDisplayMode::RGB)
		{
			return ComponentIndex >= 2;
		}
	}

	return bDefaultValue;
}

void SColorGradingColorWheel::UpdateComponentDynamicSliderMinValue(float NewValue, TWeakPtr<SWidget> SourceWidget, bool bIsOriginator, bool bUpdateOnlyIfLower)
{
	if (!ComponentSliderDynamicMinValue.IsSet() || (NewValue < ComponentSliderDynamicMinValue.GetValue() && bUpdateOnlyIfLower) || !bUpdateOnlyIfLower)
	{
		ComponentSliderDynamicMinValue = NewValue;
	}
}

void SColorGradingColorWheel::UpdateComponentDynamicSliderMaxValue(float NewValue, TWeakPtr<SWidget> SourceWidget, bool bIsOriginator, bool bUpdateOnlyIfHigher)
{
	if (!ComponentSliderDynamicMaxValue.IsSet() || (NewValue > ComponentSliderDynamicMaxValue.GetValue() && bUpdateOnlyIfHigher) || !bUpdateOnlyIfHigher)
	{
		ComponentSliderDynamicMaxValue = NewValue;
	}
}

TOptional<float> SColorGradingColorWheel::GetComponentMaxValue(TOptional<float> DefaultValue, uint32 ComponentIndex) const
{
	if (ColorDisplayMode.Get(UE::ColorGrading::EColorGradingColorDisplayMode::RGB) != UE::ColorGrading::EColorGradingColorDisplayMode::RGB)
	{
		if (ComponentIndex == 0)
		{
			return 359.0f;
		}
		else if (ComponentIndex == 1)
		{
			return 1.0f;
		}
	}

	return DefaultValue;
}

TOptional<float> SColorGradingColorWheel::GetComponentMinSliderValue(TOptional<float> DefaultValue, uint32 ComponentIndex) const
{
	if (ColorDisplayMode.Get(UE::ColorGrading::EColorGradingColorDisplayMode::RGB) != UE::ColorGrading::EColorGradingColorDisplayMode::RGB)
	{
		return 0.0f;
	}

	return ComponentSliderDynamicMinValue.IsSet() ? ComponentSliderDynamicMinValue.GetValue() : DefaultValue;
}

TOptional<float> SColorGradingColorWheel::GetComponentMaxSliderValue(TOptional<float> DefaultValue, uint32 ComponentIndex) const
{
	if (ColorDisplayMode.Get(UE::ColorGrading::EColorGradingColorDisplayMode::RGB) != UE::ColorGrading::EColorGradingColorDisplayMode::RGB)
	{
		if (ComponentIndex == 0)
		{
			return 359.0f;
		}
		else if (ComponentIndex == 1)
		{
			return 1.0f;
		}
	}

	return ComponentSliderDynamicMaxValue.IsSet() ? ComponentSliderDynamicMaxValue : DefaultValue;
}

float SColorGradingColorWheel::GetComponentSliderDeltaValue(float DefaultValue, uint32 ComponentIndex) const
{
	if (ComponentIndex == 0 && ColorDisplayMode.Get(UE::ColorGrading::EColorGradingColorDisplayMode::RGB) != UE::ColorGrading::EColorGradingColorDisplayMode::RGB)
	{
		return 1.0f;
	}

	return DefaultValue;
}

TOptional<float> SColorGradingColorWheel::GetMetadataMinValue() const
{
	if (ColorPropertyMetadata.IsSet())
	{
		return ColorPropertyMetadata->MinValue;
	}

	return TOptional<float>();
}

TOptional<float> SColorGradingColorWheel::GetMetadataMaxValue() const
{
	if (ColorPropertyMetadata.IsSet())
	{
		return ColorPropertyMetadata->MaxValue;
	}

	return TOptional<float>();
}

TOptional<float> SColorGradingColorWheel::GetMetadataSliderMinValue() const
{
	if (ColorPropertyMetadata.IsSet())
	{
		return ColorPropertyMetadata->SliderMinValue;
	}

	return TOptional<float>();
}

TOptional<float> SColorGradingColorWheel::GetMetadataSliderMaxValue() const
{
	if (ColorPropertyMetadata.IsSet())
	{
		return ColorPropertyMetadata->SliderMaxValue;
	}

	return TOptional<float>();
}

float SColorGradingColorWheel::GetMetadataDelta() const
{
	if (ColorPropertyMetadata.IsSet())
	{
		return ColorPropertyMetadata->Delta;
	}

	return 0.0f;
}

float SColorGradingColorWheel::GetMetadataShiftMultiplier() const
{
	if (ColorPropertyMetadata.IsSet())
	{
		return ColorPropertyMetadata->ShiftMultiplier;
	}

	return 10.f;
}

float SColorGradingColorWheel::GetMetadataCtrlMultiplier() const
{
	if (ColorPropertyMetadata.IsSet())
	{
		return ColorPropertyMetadata->CtrlMultiplier;
	}

	return 0.1f;
}

bool SColorGradingColorWheel::GetMetadataSupportDynamicSliderMinValue() const
{
	if (ColorPropertyMetadata.IsSet())
	{
		return ColorPropertyMetadata->bSupportDynamicSliderMinValue;
	}

	return false;
}

bool SColorGradingColorWheel::GetMetadataSupportDynamicSliderMaxValue() const
{
	if (ColorPropertyMetadata.IsSet())
	{
		return ColorPropertyMetadata->bSupportDynamicSliderMaxValue;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE