// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Customizations/MathStructCustomizations.h"
#include "EditorUndoClient.h"
#include "Util/TrackedVector4PropertyHandle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/ColorGrading/SColorGradingPicker.h"

#include "ColorGradingPanelState.h"

class IPropertyHandle;
class SBox;

/** A widget which encapsulates a color picker and numeric sliders for each color component, hooked up to a color property handle */
class SColorGradingColorWheel : public SCompoundWidget, public FEditorUndoClient
{
public:
	struct FColorPropertyMetadata
	{
		UE::ColorGrading::EColorGradingModes ColorGradingMode = UE::ColorGrading::EColorGradingModes::Invalid;
		TOptional<float> MinValue = TOptional<float>();
		TOptional<float> MaxValue = TOptional<float>();
		TOptional<float> SliderMinValue = TOptional<float>();
		TOptional<float> SliderMaxValue = TOptional<float>();
		float SliderExponent = 1.0f;
		float Delta = 0.0f;
		int32 LinearDeltaSensitivity = 0;
		float ShiftMultiplier = 10.f;
		float CtrlMultiplier = 0.1f;
		bool bSupportDynamicSliderMaxValue = false;
		bool bSupportDynamicSliderMinValue = false;
	};

public:
	SLATE_BEGIN_ARGS(SColorGradingColorWheel)
	{}
		SLATE_ATTRIBUTE(UE::ColorGrading::EColorGradingColorDisplayMode, ColorDisplayMode)
		SLATE_ARGUMENT(TSharedPtr<SWidget>, HeaderContent)
	SLATE_END_ARGS()

	SColorGradingColorWheel();
	virtual ~SColorGradingColorWheel();

	void Construct(const FArguments& InArgs);

	/** Sets the property handle for the color property to edit with this color wheel */
	void SetColorPropertyHandle(TSharedPtr<IPropertyHandle> InColorPropertyHandle);

	/** Sets the widget to display as the header of the color wheel */
	void SetHeaderContent(const TSharedRef<SWidget>& HeaderContent);

	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	// End of FEditorUndoClient

	/** Called when any property changes */
	void OnPropertyValueChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);

private:
	TSharedRef<SWidget> CreateColorGradingPicker();
	TSharedRef<SWidget> CreateColorComponentSliders();

	FColorPropertyMetadata GetColorPropertyMetadata() const;

	bool IsPropertyEnabled() const;
	EVisibility GetSlidersVisibility() const;
	EVisibility GetShortLayoutVisibility() const;
	EVisibility GetTallLayoutVisibility() const;
	int32 GetMaxWheelWidth() const;
	bool ShouldUseTallLayout() const;

	bool GetColor(FVector4& OutCurrentColor);
	void CommitColor(FVector4& NewValue, bool bShouldCommitValueChanges);
	void TransactColorValue();
	void RecalculateHSVColor();

	void BeginUsingColorPickerSlider();
	void EndUsingColorPickerSlider();
	void BeginUsingComponentSlider(uint32 ComponentIndex);
	void EndUsingComponentSlider(float NewValue, uint32 ComponentIndex);

	UE::ColorGrading::EColorGradingComponent GetComponent(uint32 ComponentIndex) const;
	TOptional<float> GetComponentValue(uint32 ComponentIndex) const;
	void SetComponentValue(float NewValue, uint32 ComponentIndex);

	bool ComponentSupportsDynamicSliderValue(bool bDefaultValue, uint32 ComponentIndex) const;
	void UpdateComponentDynamicSliderMinValue(float NewValue, TWeakPtr<SWidget> SourceWidget, bool bIsOriginator, bool bUpdateOnlyIfLower);
	void UpdateComponentDynamicSliderMaxValue(float NewValue, TWeakPtr<SWidget> SourceWidget, bool bIsOriginator, bool bUpdateOnlyIfHigher);

	TOptional<float> GetComponentMaxValue(TOptional<float> DefaultValue, uint32 ComponentIndex) const;
	TOptional<float> GetComponentMinSliderValue(TOptional<float> DefaultValue, uint32 ComponentIndex) const;
	TOptional<float> GetComponentMaxSliderValue(TOptional<float> DefaultValue, uint32 ComponentIndex) const;
	float GetComponentSliderDeltaValue(float DefaultValue, uint32 ComponentIndex) const;

	TOptional<float> GetMetadataMinValue() const;
	TOptional<float> GetMetadataMaxValue() const;
	TOptional<float> GetMetadataSliderMinValue() const;
	TOptional<float> GetMetadataSliderMaxValue() const;
	float GetMetadataDelta() const;
	float GetMetadataShiftMultiplier() const;
	float GetMetadataCtrlMultiplier() const;
	bool GetMetadataSupportDynamicSliderMinValue() const;
	bool GetMetadataSupportDynamicSliderMaxValue() const;

private:
	/** Padding applied to the whole column */
	const FVector2f ColumnPadding = FVector2f(16.f, 8.f);

	TSharedPtr<UE::ColorGrading::SColorGradingPicker> ColorGradingPicker;
	TSharedPtr<SBox> HeaderBox;
	TSharedPtr<SBox> ColorPickerBox;
	TSharedPtr<SBox> ColorSlidersBox;

	/** The property handle of the linear color property being edited */
	FTrackedVector4PropertyHandle ColorPropertyHandle;

	/** The metadata of the color property */
	TOptional<FColorPropertyMetadata> ColorPropertyMetadata;

	/** Attribute for the color mode type the color wheel is presenting the color components in */
	TAttribute<UE::ColorGrading::EColorGradingColorDisplayMode> ColorDisplayMode;

	/** Stored current min value of the color component numeric sliders */
	TOptional<float> ComponentSliderDynamicMinValue;

	/** Stored current max value of the color component numeric sliders */
	TOptional<float> ComponentSliderDynamicMaxValue;

	/** Indicates that the color picker slider is currently being used to change the color on the color picker */
	bool bIsUsingColorPickerSlider = false;

	/** Indicates that a component's numeric slider is currently being used to change the color */
	bool bIsUsingComponentSlider = false;

	/**
	 * The current color in HSV space.
	 * Stored separately so that hue/saturation adjustments aren't lost when the color is 0.
	 */
	FLinearColor CurrentHSVColor;
};