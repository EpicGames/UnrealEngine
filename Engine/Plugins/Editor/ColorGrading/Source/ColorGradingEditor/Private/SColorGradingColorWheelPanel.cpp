// Copyright Epic Games, Inc. All Rights Reserved.

#include "SColorGradingColorWheelPanel.h"

#include "ColorGradingCommands.h"
#include "ColorGradingPanelState.h"
#include "SColorGradingColorWheel.h"
#include "DetailView/SColorGradingDetailView.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSplitter.h"
 
#define LOCTEXT_NAMESPACE "ColorGradingEditor"

SColorGradingColorWheelPanel::~SColorGradingColorWheelPanel()
{
	if (ColorGradingDataModel)
	{
		ColorGradingDataModel->OnColorGradingGroupSelectionChanged().RemoveAll(this);
		ColorGradingDataModel->OnColorGradingElementSelectionChanged().RemoveAll(this);
	}
}

void SColorGradingColorWheelPanel::Construct(const FArguments& InArgs)
{
	ColorGradingDataModel = InArgs._ColorGradingDataModelSource;

	if (ColorGradingDataModel)
	{
		ColorGradingDataModel->OnColorGradingGroupSelectionChanged().AddSP(this, &SColorGradingColorWheelPanel::OnColorGradingGroupSelectionChanged);
		ColorGradingDataModel->OnColorGradingElementSelectionChanged().AddSP(this, &SColorGradingColorWheelPanel::OnColorGradingElementSelectionChanged);
	}

	ColorWheels.AddDefaulted(NumColorWheels);

	for (int32 Index = 0; Index < NumColorWheels; ++Index)
    ChildSlot
	[
		SNew(SVerticalBox)

		// Message indicating that multi select is unavailable in this panel
		+ SVerticalBox::Slot()
		[
			SNew(SBox)
			.Visibility(this, &SColorGradingColorWheelPanel::GetMultiSelectWarningVisibility)
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("MultiSelectWarning", "Multi-select editing is unavailable in the Color Grading panel."))
			]
		]

		// Color wheel panel
		+ SVerticalBox::Slot()
		[
			SNew(SSplitter)
			.Orientation(Orient_Horizontal)
			.PhysicalSplitterHandleSize(2.0f)
			.HitDetectionSplitterHandleSize(5.0f)
			.Style(FAppStyle::Get(), "DetailsView.Splitter")
			.Visibility(this, &SColorGradingColorWheelPanel::GetColorWheelPanelVisibility)

			+ SSplitter::Slot()
			.Value(0.8f)
			[
				SNew(SVerticalBox)

				// Toolbar slot
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(6, 4)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SAssignNew(ColorGradingGroupPropertyBox, SBox)
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(SSpacer)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SAssignNew(ColorGradingElementsToolBarBox, SHorizontalBox)
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(SSpacer)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						MakeColorDisplayModeCheckbox()
					]
				]	

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SSeparator)
					.Thickness(2.0f)
				]


				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.HAlign(HAlign_Fill)
					[
						SAssignNew(ColorWheels[0], SColorGradingColorWheel)
						.ColorDisplayMode(UE::ColorGrading::EColorGradingColorDisplayMode::RGB) // Offset wheel is locked to RGB mode

					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SSeparator)
						.Orientation(EOrientation::Orient_Vertical)
						.Thickness(2.0f)
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.HAlign(HAlign_Fill)
					[
						SAssignNew(ColorWheels[1], SColorGradingColorWheel)
						.ColorDisplayMode(this, &SColorGradingColorWheelPanel::GetColorDisplayMode)

					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SSeparator)
						.Orientation(EOrientation::Orient_Vertical)
						.Thickness(2.0f)
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.HAlign(HAlign_Fill)
					[
						SAssignNew(ColorWheels[2], SColorGradingColorWheel)
						.ColorDisplayMode(this, &SColorGradingColorWheelPanel::GetColorDisplayMode)

					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SSeparator)
						.Orientation(EOrientation::Orient_Vertical)
						.Thickness(2.0f)
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.HAlign(HAlign_Fill)
					[
						SAssignNew(ColorWheels[3], SColorGradingColorWheel)
						.ColorDisplayMode(this, &SColorGradingColorWheelPanel::GetColorDisplayMode)

					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SSeparator)
						.Orientation(EOrientation::Orient_Vertical)
						.Thickness(2.0f)
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.HAlign(HAlign_Fill)
					[
						SAssignNew(ColorWheels[4], SColorGradingColorWheel)
						.ColorDisplayMode(this, &SColorGradingColorWheelPanel::GetColorDisplayMode)
					]
				]
			]

			+ SSplitter::Slot()
			.Value(0.2f)
			[
				SAssignNew(DetailView, SColorGradingDetailView)
				.PropertyRowGeneratorSource(ColorGradingDataModel->GetPropertyRowGenerator())
				.OnFilterDetailTreeNode(this, &SColorGradingColorWheelPanel::FilterDetailTreeNode)
			]
		]
	];
}

void SColorGradingColorWheelPanel::Refresh()
{
	if (ColorGradingDataModel)
	{
		if (FColorGradingEditorDataModel::FColorGradingGroup* ColorGradingGroup = ColorGradingDataModel->GetSelectedColorGradingGroup())
		{
			FillColorGradingGroupProperty(*ColorGradingGroup);
			FillColorGradingElementsToolBar(ColorGradingGroup->ColorGradingElements);

			if (const FColorGradingEditorDataModel::FColorGradingElement* ColorGradingElement = ColorGradingDataModel->GetSelectedColorGradingElement())
			{
				FillColorWheels(*ColorGradingElement);
			}
			else
			{
				ClearColorWheels();
			}
		}
		else
		{
			ClearColorGradingGroupProperty();
			ClearColorGradingElementsToolBar();
			ClearColorWheels();
		}

		DetailView->Refresh();
	}
}

void SColorGradingColorWheelPanel::GetPanelState(FColorGradingPanelState& OutPanelState)
{
	OutPanelState.ColorDisplayMode = ColorDisplayMode;
}

void SColorGradingColorWheelPanel::SetPanelState(const FColorGradingPanelState& InPanelState)
{
	// TODO: These could also be output to a config file to be stored between runs
	ColorDisplayMode = InPanelState.ColorDisplayMode;
}

TSharedRef<SWidget> SColorGradingColorWheelPanel::MakeColorDisplayModeCheckbox()
{
	using SDisplayModeControl = SSegmentedControl<UE::ColorGrading::EColorGradingColorDisplayMode>;
	return SNew(SDisplayModeControl)
		.OnValueChanged(this, &SColorGradingColorWheelPanel::OnColorDisplayModeChanged)
		.Value(this, &SColorGradingColorWheelPanel::GetColorDisplayMode)
		.UniformPadding(FMargin(16.0f, 2.0f))
		
		+ SDisplayModeControl::Slot(UE::ColorGrading::EColorGradingColorDisplayMode::RGB)
		.Text(this, &SColorGradingColorWheelPanel::GetColorDisplayModeLabel, UE::ColorGrading::EColorGradingColorDisplayMode::RGB)
		.ToolTip(this, &SColorGradingColorWheelPanel::GetColorDisplayModeToolTip, UE::ColorGrading::EColorGradingColorDisplayMode::RGB)
		
		+ SDisplayModeControl::Slot(UE::ColorGrading::EColorGradingColorDisplayMode::HSV)
		.Text(this, &SColorGradingColorWheelPanel::GetColorDisplayModeLabel, UE::ColorGrading::EColorGradingColorDisplayMode::HSV)
		.ToolTip(this, &SColorGradingColorWheelPanel::GetColorDisplayModeToolTip, UE::ColorGrading::EColorGradingColorDisplayMode::HSV);
}

void SColorGradingColorWheelPanel::FillColorGradingGroupProperty(const FColorGradingEditorDataModel::FColorGradingGroup& ColorGradingGroup)
{
	if (ColorGradingGroupPropertyBox.IsValid())
	{
		TSharedRef<SHorizontalBox> PropertyNameBox = SNew(SHorizontalBox);

		if (ColorGradingGroup.EditConditionPropertyHandle.IsValid())
		{
			if (TSharedPtr<IDetailTreeNode> EditConditionTreeNode = ColorGradingDataModel->GetPropertyRowGenerator()->FindTreeNode(ColorGradingGroup.EditConditionPropertyHandle))
			{
				FNodeWidgets EditConditionWidgets = EditConditionTreeNode->CreateNodeWidgets();

				if (EditConditionWidgets.ValueWidget.IsValid())
				{
					PropertyNameBox->AddSlot()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						.Padding(2, 0, 4, 0)
						.AutoWidth()
						[
							EditConditionWidgets.ValueWidget.ToSharedRef()
						];
				}
			}
		}

		TSharedRef<SWidget> GroupHeaderWidget = ColorGradingGroup.GroupHeaderWidget.IsValid()
			? ColorGradingGroup.GroupHeaderWidget.ToSharedRef()
			: SNew(STextBlock).Text(ColorGradingGroup.DisplayName).Font(FAppStyle::Get().GetFontStyle("NormalFontBold"));

		PropertyNameBox->AddSlot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(2, 0, 0, 0)
			[
				GroupHeaderWidget
			];

		ColorGradingGroupPropertyBox->SetContent(PropertyNameBox);
	}
}

void SColorGradingColorWheelPanel::ClearColorGradingGroupProperty()
{
	ColorGradingGroupPropertyBox->SetContent(SNullWidget::NullWidget);
}

void SColorGradingColorWheelPanel::FillColorGradingElementsToolBar(const TArray<FColorGradingEditorDataModel::FColorGradingElement>& ColorGradingElements)
{
	ColorGradingElementsToolBarBox->ClearChildren();

	for (const FColorGradingEditorDataModel::FColorGradingElement& Element : ColorGradingElements)
	{
		ColorGradingElementsToolBarBox->AddSlot()
			.AutoWidth()
			.Padding(2.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "DetailsView.SectionButton")
				.OnCheckStateChanged(this, &SColorGradingColorWheelPanel::OnColorGradingElementCheckedChanged, Element.DisplayName)
				.IsChecked(this, &SColorGradingColorWheelPanel::IsColorGradingElementSelected, Element.DisplayName)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "SmallText")
					.Text(Element.DisplayName)
				]
			];
	}
}

void SColorGradingColorWheelPanel::ClearColorGradingElementsToolBar()
{
	ColorGradingElementsToolBarBox->ClearChildren();
}

void SColorGradingColorWheelPanel::FillColorWheels(const FColorGradingEditorDataModel::FColorGradingElement& ColorGradingElement)
{
	auto FillColorWheel = [this](const TSharedPtr<SColorGradingColorWheel>& ColorWheel,
		const TSharedPtr<IPropertyHandle>& PropertyHandle,
		const TOptional<FResetToDefaultOverride>& ResetToDefaultOverride)
	{
		if (ColorWheel.IsValid())
		{
			ColorWheel->SetColorPropertyHandle(PropertyHandle);
			ColorWheel->SetHeaderContent(CreateColorWheelHeaderWidget(PropertyHandle, ResetToDefaultOverride));
		}
	};

	FillColorWheel(ColorWheels[0], ColorGradingElement.SaturationPropertyHandle, ColorGradingElement.SaturationResetToDefaultOverride);
	FillColorWheel(ColorWheels[1], ColorGradingElement.ContrastPropertyHandle, ColorGradingElement.ContrastResetToDefaultOverride);
	FillColorWheel(ColorWheels[2], ColorGradingElement.GammaPropertyHandle, ColorGradingElement.GammaResetToDefaultOverride);
	FillColorWheel(ColorWheels[3], ColorGradingElement.GainPropertyHandle, ColorGradingElement.GainResetToDefaultOverride);
	FillColorWheel(ColorWheels[4], ColorGradingElement.OffsetPropertyHandle, ColorGradingElement.OffsetResetToDefaultOverride);
}

void SColorGradingColorWheelPanel::ClearColorWheels()
{
	for (const TSharedPtr<SColorGradingColorWheel>& ColorWheel : ColorWheels)
	{
		if (ColorWheel.IsValid())
		{
			ColorWheel->SetColorPropertyHandle(nullptr);
			ColorWheel->SetHeaderContent(SNullWidget::NullWidget);
		}
	};
}

TSharedRef<SWidget> SColorGradingColorWheelPanel::CreateColorWheelHeaderWidget(const TSharedPtr<IPropertyHandle>& ColorPropertyHandle, const TOptional<FResetToDefaultOverride>& ResetToDefaultOverride)
{
	if (TSharedPtr<IDetailTreeNode> TreeNode = ColorGradingDataModel->GetPropertyRowGenerator()->FindTreeNode(ColorPropertyHandle))
	{
		FNodeWidgets NodeWidgets = TreeNode->CreateNodeWidgets();

		TSharedRef<SHorizontalBox> PropertyNameBox = SNew(SHorizontalBox);

		if (NodeWidgets.EditConditionWidget.IsValid())
		{
			PropertyNameBox->AddSlot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(2, 0, 0, 0)
				.AutoWidth()
				[
					NodeWidgets.EditConditionWidget.ToSharedRef()
				];
		}

		if (NodeWidgets.NameWidget.IsValid())
		{
			PropertyNameBox->AddSlot()
				.HAlign(NodeWidgets.NameWidgetLayoutData.HorizontalAlignment)
				.VAlign(NodeWidgets.NameWidgetLayoutData.VerticalAlignment)
				.Padding(2, 0, 0, 0)
				[
					NodeWidgets.NameWidget.ToSharedRef()
				];
			PropertyNameBox->AddSlot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.FillWidth(1.0f)
				[
					SNew(SBox)
					.MinDesiredWidth(22.0f)
					[
						CreateColorPropertyExtensions(ColorPropertyHandle, TreeNode, ResetToDefaultOverride)
					]
				];
		}

		return PropertyNameBox;
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SColorGradingColorWheelPanel::CreateColorPropertyExtensions(const TSharedPtr<IPropertyHandle>& ColorPropertyHandle,
	const TSharedPtr<IDetailTreeNode>& DetailTreeNode,
	const TOptional<FResetToDefaultOverride>& ResetToDefaultOverride)
{
	// Use a weak pointer to pass into delegates
	TWeakPtr<IPropertyHandle> WeakColorPropertyHandle = ColorPropertyHandle;

	FPropertyRowExtensionButton ResetToDefaultButton;
	ResetToDefaultButton.Label = NSLOCTEXT("PropertyEditor", "ResetToDefault", "Reset to Default");
	ResetToDefaultButton.UIAction = FUIAction(
		FExecuteAction::CreateLambda([WeakColorPropertyHandle, ResetToDefaultOverride]
		{
			if (WeakColorPropertyHandle.IsValid())
			{
				if (ResetToDefaultOverride.IsSet())
				{
					WeakColorPropertyHandle.Pin()->ExecuteCustomResetToDefault(ResetToDefaultOverride.GetValue());
				}
				else
				{
					WeakColorPropertyHandle.Pin()->ResetToDefault();
				}
			}
		}),
		FCanExecuteAction::CreateLambda([WeakColorPropertyHandle]
		{
			const bool bIsEditable = WeakColorPropertyHandle.Pin()->IsEditable();
			return bIsEditable;
		}),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateLambda([WeakColorPropertyHandle]
		{
			bool bShowResetToDefaultButton = false;
			if (WeakColorPropertyHandle.IsValid())
			{
				if (!WeakColorPropertyHandle.Pin()->HasMetaData("NoResetToDefault") && !WeakColorPropertyHandle.Pin()->GetInstanceMetaData("NoResetToDefault"))
				{
					bShowResetToDefaultButton = WeakColorPropertyHandle.Pin()->CanResetToDefault();
				}
			}

			return bShowResetToDefaultButton;
		})
	);

	ResetToDefaultButton.Icon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "PropertyWindow.DiffersFromDefault");
	ResetToDefaultButton.ToolTip = NSLOCTEXT("PropertyEditor", "ResetToDefaultPropertyValueToolTip", "Reset this property to its default value.");

	// Add any global row extensions that are registered for the color property
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FOnGenerateGlobalRowExtensionArgs Args;
	Args.OwnerTreeNode = DetailTreeNode;
	Args.PropertyHandle = ColorPropertyHandle;

	TArray<FPropertyRowExtensionButton> ExtensionButtons;
	PropertyEditorModule.GetGlobalRowExtensionDelegate().Broadcast(Args, ExtensionButtons);

	FSlimHorizontalToolBarBuilder ToolbarBuilder(TSharedPtr<FUICommandList>(), FMultiBoxCustomization::None);
	ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);
	ToolbarBuilder.SetStyle(&FAppStyle::Get(), "DetailsView.ExtensionToolBar");
	ToolbarBuilder.SetIsFocusable(false);

	// Always show reset to default. The other button are shown if there is enough space.
	for (const FPropertyRowExtensionButton& Extension : ExtensionButtons)
	{
		ToolbarBuilder.AddToolBarButton(
			Extension.UIAction, NAME_None, Extension.Label, Extension.ToolTip, Extension.Icon,
			EUserInterfaceActionType::Button, NAME_None, {}, {},
			FMenuEntryResizeParams{ .AllowClipping = true }
			);
	}
	// Add the reset button last so it's always the right-most widget.
	ToolbarBuilder.AddToolBarButton(
		ResetToDefaultButton.UIAction, NAME_None, ResetToDefaultButton.Label, ResetToDefaultButton.ToolTip, ResetToDefaultButton.Icon,
		EUserInterfaceActionType::Button, NAME_None, {}, {},
		FMenuEntryResizeParams{ .AllowClipping = false }
		);

	return ToolbarBuilder.MakeWidget();
}

bool SColorGradingColorWheelPanel::FilterDetailTreeNode(const TSharedRef<IDetailTreeNode>& InDetailTreeNode)
{
	if (ColorGradingDataModel)
	{
		if (FColorGradingEditorDataModel::FColorGradingGroup* ColorGradingGroup = ColorGradingDataModel->GetSelectedColorGradingGroup())
		{
			// Filter out any categories that are not configured by the data model to be displayed in the details section or subsection.
			// All other nodes (which will be any child of the category), should be displayed.
			if (InDetailTreeNode->GetNodeType() == EDetailNodeType::Category)
			{
				return ColorGradingGroup->DetailsViewCategories.Contains(InDetailTreeNode->GetNodeName());
			}
			else
			{
				return true;
			}
		}
	}

	return false;
}

void SColorGradingColorWheelPanel::OnColorGradingGroupSelectionChanged()
{
	Refresh();
}

void SColorGradingColorWheelPanel::OnColorGradingElementSelectionChanged()
{
	if (const FColorGradingEditorDataModel::FColorGradingElement* ColorGradingElement = ColorGradingDataModel->GetSelectedColorGradingElement())
	{
		FillColorWheels(*ColorGradingElement);
	}
	else
	{
		ClearColorWheels();
	}
}

void SColorGradingColorWheelPanel::OnColorGradingElementCheckedChanged(ECheckBoxState State, FText ElementName)
{
	if (State == ECheckBoxState::Checked && ColorGradingDataModel)
	{
		if (FColorGradingEditorDataModel::FColorGradingGroup* ColorGradingGroup = ColorGradingDataModel->GetSelectedColorGradingGroup())
		{
			int32 ColorGradingElementIndex = ColorGradingGroup->ColorGradingElements.IndexOfByPredicate([=](const FColorGradingEditorDataModel::FColorGradingElement& Element)
			{
				return Element.DisplayName.CompareTo(ElementName) == 0;
			});

			ColorGradingDataModel->SetSelectedColorGradingElement(ColorGradingElementIndex);
		}
	}
}

ECheckBoxState SColorGradingColorWheelPanel::IsColorGradingElementSelected(FText ElementName) const
{
	if (ColorGradingDataModel)
	{
		if (const FColorGradingEditorDataModel::FColorGradingElement* ColorGradingElement = ColorGradingDataModel->GetSelectedColorGradingElement())
		{
			if (ColorGradingElement->DisplayName.CompareTo(ElementName) == 0)
			{
				return ECheckBoxState::Checked;
			}
		}
	}

	return ECheckBoxState::Unchecked;
}

EVisibility SColorGradingColorWheelPanel::GetColorWheelPanelVisibility() const
{
	bool bHasObject = ColorGradingDataModel && ColorGradingDataModel->GetPropertyRowGenerator()->GetSelectedObjects().Num() == 1;
	return bHasObject ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SColorGradingColorWheelPanel::GetMultiSelectWarningVisibility() const
{
	bool bHasMultipleObjects = ColorGradingDataModel && ColorGradingDataModel->GetPropertyRowGenerator()->GetSelectedObjects().Num() > 1;
	return bHasMultipleObjects ? EVisibility::Visible : EVisibility::Collapsed;
}

void SColorGradingColorWheelPanel::OnColorDisplayModeChanged(UE::ColorGrading::EColorGradingColorDisplayMode InColorDisplayMode)
{
	ColorDisplayMode = InColorDisplayMode;
}

FText SColorGradingColorWheelPanel::GetColorDisplayModeLabel(UE::ColorGrading::EColorGradingColorDisplayMode InColorDisplayMode) const
{
	FText Text;

	switch (InColorDisplayMode)
	{
	case UE::ColorGrading::EColorGradingColorDisplayMode::RGB: Text = LOCTEXT("ColorWheel_RGBColorDisplayModeLabel", "RGB"); break;
	case UE::ColorGrading::EColorGradingColorDisplayMode::HSV: Text = LOCTEXT("ColorWheel_HSVColorDisplayModeLabel", "HSV"); break;
	}

	return Text;
}

FText SColorGradingColorWheelPanel::GetColorDisplayModeToolTip(UE::ColorGrading::EColorGradingColorDisplayMode InColorDisplayMode) const
{
	FText Text;

	switch (InColorDisplayMode)
	{
	case UE::ColorGrading::EColorGradingColorDisplayMode::RGB: Text = LOCTEXT("ColorWheel_RGBColorDisplayModeToolTip", "Change to RGB color mode"); break;
	case UE::ColorGrading::EColorGradingColorDisplayMode::HSV: Text = LOCTEXT("ColorWheel_HSVColorDisplayModeToolTip", "Change to HSV color mode"); break;
	}

	return Text;
}

#undef LOCTEXT_NAMESPACE