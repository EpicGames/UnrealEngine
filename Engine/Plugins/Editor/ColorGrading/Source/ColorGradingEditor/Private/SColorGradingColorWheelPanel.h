// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ColorGradingPanelState.h"
#include "ColorGradingEditorDataModel.h"
#include "SColorGradingColorWheel.h"
#include "Styling/SlateTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class IDetailTreeNode;
class IPropertyRowGenerator;
class SBox;
class SColorGradingDetailView;
class SHorizontalBox;
struct FColorGradingPanelState;

/** A panel that contains up to five color wheels (for saturation, contrast, gamma, gain, and offset) as well as a details view for extra, non-color properties */
class SColorGradingColorWheelPanel : public SCompoundWidget
{
private:
	/** The number of color wheels the color wheel panel displays (for saturation, contrast, gamma, gain, and offset) */
	static const uint32 NumColorWheels = 5;

public:
	virtual ~SColorGradingColorWheelPanel() override;

	SLATE_BEGIN_ARGS(SColorGradingColorWheelPanel) {}
		SLATE_ARGUMENT(TSharedPtr<FColorGradingEditorDataModel>, ColorGradingDataModelSource)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Regenerates the color wheel panel from the current state of the data model source */
	void Refresh();

	/** Adds the state of the color wheel panel to the specified drawer state */
	void GetPanelState(FColorGradingPanelState& OutPanelState);

	/** Sets the state of the color wheel panel from the specified drawer state */
	void SetPanelState(const FColorGradingPanelState& InPanelState);

private:
	TSharedRef<SWidget> MakeColorDisplayModeCheckbox();

	void FillColorGradingGroupProperty(const FColorGradingEditorDataModel::FColorGradingGroup& ColorGradingGroup);
	void ClearColorGradingGroupProperty();

	void FillColorGradingElementsToolBar(const TArray<FColorGradingEditorDataModel::FColorGradingElement>& ColorGradingElements);
	void ClearColorGradingElementsToolBar();

	void FillColorWheels(const FColorGradingEditorDataModel::FColorGradingElement& ColorGradingElement);
	void ClearColorWheels();

	TSharedRef<SWidget> CreateColorWheelHeaderWidget(const TSharedPtr<IPropertyHandle>& ColorPropertyHandle, const TOptional<FResetToDefaultOverride>& ResetToDefaultOverride);
	TSharedRef<SWidget> CreateColorPropertyExtensions(const TSharedPtr<IPropertyHandle>& ColorPropertyHandle, const TSharedPtr<IDetailTreeNode>& DetailTreeNode, const TOptional<FResetToDefaultOverride>& ResetToDefaultOverride);

	bool FilterDetailTreeNode(const TSharedRef<IDetailTreeNode>& InDetailTreeNode);

	void OnColorGradingGroupSelectionChanged();
	void OnColorGradingElementSelectionChanged();

	void OnColorGradingElementCheckedChanged(ECheckBoxState State, FText ElementName);

	ECheckBoxState IsColorGradingElementSelected(FText ElementName) const;
	EVisibility GetColorWheelPanelVisibility() const;
	EVisibility GetMultiSelectWarningVisibility() const;

	UE::ColorGrading::EColorGradingColorDisplayMode GetColorDisplayMode() const { return ColorDisplayMode; }
	void OnColorDisplayModeChanged(UE::ColorGrading::EColorGradingColorDisplayMode InColorDisplayMode);
	FText GetColorDisplayModeLabel(UE::ColorGrading::EColorGradingColorDisplayMode InColorDisplayMode) const;
	FText GetColorDisplayModeToolTip(UE::ColorGrading::EColorGradingColorDisplayMode InColorDisplayMode) const;

private:
	/** The color grading data model that the panel is displaying */
	TSharedPtr<FColorGradingEditorDataModel> ColorGradingDataModel;

	TSharedPtr<SBox> ColorGradingGroupPropertyBox;
	TSharedPtr<SHorizontalBox> ColorGradingElementsToolBarBox;

	TArray<TSharedPtr<SColorGradingColorWheel>> ColorWheels;

	TSharedPtr<SColorGradingDetailView> DetailView;

	/** The currently selected color grading group */
	int32 SelectedColorGradingGroup = INDEX_NONE;

	/** The current color display mode for the color wheels */
	UE::ColorGrading::EColorGradingColorDisplayMode ColorDisplayMode = UE::ColorGrading::EColorGradingColorDisplayMode::RGB;
};