// Copyright Epic Games, Inc. All Rights Reserved.

#include "SColorGradingPanel.h"

#include "ColorGradingCommands.h"
#include "ColorGradingEditorDataModel.h"
#include "ColorGradingMixerObjectFilter.h"
#include "ColorGradingMixerObjectFilterRegistry.h"
#include "IColorGradingEditor.h"
#include "SColorGradingColorWheelPanel.h"

#include "ActorTreeItem.h"
#include "Algo/Compare.h"
#include "ComponentTreeItem.h"
#include "Editor.h"
#include "Engine/Blueprint.h"
#include "Engine/Engine.h"
#include "EngineUtils.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LevelEditorSubsystem.h"
#include "Views/List/ObjectMixerEditorList.h"
#include "Views/List/RowTypes/ObjectMixerEditorListRowActor.h"
#include "Views/List/SObjectMixerEditorList.h"
#include "Views/List/RowTypes/ObjectMixerEditorListRowUObject.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "ColorGradingEditor"

SColorGradingPanel::~SColorGradingPanel()
{
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);

	if (GEngine)
	{
		GEngine->OnLevelActorAdded().RemoveAll(this);
		GEngine->OnLevelActorDeleted().RemoveAll(this);
	}

	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

void SColorGradingPanel::Construct(const FArguments& InArgs)
{
	bIsInDrawer = InArgs._IsInDrawer;
	DockCallback = InArgs._OnDocked;
	OverrideWorld = InArgs._OverrideWorld;
	ActorFilter = InArgs._ActorFilter;

	ColorGradingDataModel = MakeShared<FColorGradingEditorDataModel>();
	ColorGradingDataModel->OnDataModelGenerated().AddSP(this, &SColorGradingPanel::OnColorGradingDataModelGenerated);

	const FName ModuleName = FName(TEXT("ColorGrading"));
	ObjectListModel = MakeShared<FObjectMixerEditorList>(ModuleName, InArgs._SelectionInterface);
	ObjectListModel->Initialize();
	ObjectListModel->SetDefaultFilterClass(UColorGradingMixerObjectFilter::StaticClass());

	TSharedRef<SWidget> ObjectListWidget = ObjectListModel->GetOrCreateWidget();
	ColorGradingObjectListView = StaticCastSharedRef<SSceneOutliner>(ObjectListWidget).ToSharedPtr();

	TSharedRef<SObjectMixerEditorList> ObjectMixerList = StaticCastSharedRef<SObjectMixerEditorList>(ObjectListWidget);
	ObjectMixerList->GetOnItemSelectionChanged().AddSP(this, &SColorGradingPanel::OnListSelectionChanged);
	ObjectMixerList->GetOnSelectionSynchronized().AddSP(this, &SColorGradingPanel::OnListSelectionSynchronized);

	ObjectMixerList->AddFilter(
		MakeShared<TSceneOutlinerPredicateFilter<FActorTreeItem>>(
			FActorTreeItem::FFilterPredicate::CreateSPLambda(this, [this](const AActor* Actor)
			{
				if (ActorFilter)
				{
					return ActorFilter(Actor);
				}

				return true;
			}),
			FSceneOutlinerFilter::EDefaultBehaviour::Pass
		)
	);

	GEditor->RegisterForUndo(this);

	RefreshColorGradingList();

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(FMargin(0.0f, 0.0f))
		[
			// Splitter to divide the object list and the color panel
			SNew(SSplitter)
			.Orientation(Orient_Horizontal)
			.PhysicalSplitterHandleSize(2.0f)

			// Splitter slot for object list
			+SSplitter::Slot()
			.Value(0.2f)
			[
				SNew(SBox)
				.Padding(FMargin(4.f))
				[
					ObjectListWidget
				]
			]

			// Splitter slot for color grading controls/details
			+SSplitter::Slot()
			.Value(0.8f)
			[
				SNew(SVerticalBox)

				// Toolbar slot for the main drawer toolbar
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0, 0, 0, 0)
				[
					SNew(SBorder)
					.Padding(FMargin(3))
					.BorderImage(bIsInDrawer ? FStyleDefaults::GetNoBrush() : FAppStyle::Get().GetBrush("Brushes.Panel"))
					[
						SNew(SBox)
						.HeightOverride(28.0f)
						[
							SNew(SHorizontalBox)

							// Slot for the color grading group toolbar
							+SHorizontalBox::Slot()
							.AutoWidth()
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							[
								SAssignNew(ColorGradingGroupToolBarBox, SHorizontalBox)
								.Visibility(this, &SColorGradingPanel::GetColorGradingGroupToolBarVisibility)
							]


							+SHorizontalBox::Slot()
							.FillWidth(1.0f)
							[
								SNew(SSpacer)
							]

							// Slot for the "Dock in Layout" button
							+SHorizontalBox::Slot()
							.AutoWidth()
							.HAlign(HAlign_Right)
							.VAlign(VAlign_Center)
							[
								CreateDockInLayoutButton()
							]
						]
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SSeparator)
					.Thickness(2.0f)
				]

				// Slot for the color panel
				+SVerticalBox::Slot()
				[
					SNew(SBorder)
					.Padding(0.f)
					.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
					[
						SAssignNew(ColorWheelPanel, SColorGradingColorWheelPanel)
						.ColorGradingDataModelSource(ColorGradingDataModel)
					]
				]
			]
		]
	];
}

void SColorGradingPanel::Refresh()
{
	FColorGradingPanelState PanelState;
	GetPanelState(PanelState);

	ColorGradingDataModel->Reset();

	RefreshColorGradingList();

	if (ColorWheelPanel.IsValid())
	{
		ColorWheelPanel->Refresh();
	}

	SetPanelState(PanelState);
}

void SColorGradingPanel::GetPanelState(FColorGradingPanelState& OutPanelState) const
{
	ColorGradingDataModel->GetPanelState(OutPanelState);

	if (ColorWheelPanel.IsValid())
	{
		ColorWheelPanel->GetPanelState(OutPanelState);
	}

	if (ColorGradingObjectListView.IsValid())
	{
		TArray<FSceneOutlinerTreeItemPtr> SelectedItems = ColorGradingObjectListView->GetSelectedItems();
		TArray<UObject*> SelectedObjects;
		TArray<UObject*> ControlledObjects;

		GetSelectedAndControlledObjects(SelectedItems, SelectedObjects, ControlledObjects);

		OutPanelState.SelectedObjects.Append(SelectedObjects);
		OutPanelState.ControlledObjects.Append(ControlledObjects);
	}
}

void SColorGradingPanel::SetPanelState(const FColorGradingPanelState& InPanelState)
{
	TArray<FSceneOutlinerTreeItemPtr> ItemsToSelect;

	for (const TWeakObjectPtr<UObject>& SelectedObject : InPanelState.SelectedObjects)
	{
		if (!SelectedObject.IsValid())
		{
			continue;
		}

		FSceneOutlinerTreeItemPtr Item = ColorGradingObjectListView->GetTreeItem(SelectedObject.Get());
		if (Item.IsValid())
		{
			ItemsToSelect.Add(Item);
		}
	}

	if (!ItemsToSelect.IsEmpty() && ColorGradingObjectListView.IsValid())
	{
		ColorGradingObjectListView->ClearSelection();
		ColorGradingObjectListView->SetItemSelection(ItemsToSelect, true);
	}

	ColorGradingDataModel->SetPanelState(InPanelState);

	if (ColorWheelPanel.IsValid())
	{
		ColorWheelPanel->SetPanelState(InPanelState);
	}
}

void SColorGradingPanel::SetSelectedObjects(const TArray<UObject*>& SelectedObjects, const TArray<UObject*>* ControlledObjects)
{
	const TArray<UObject*>& ControlledObjectsToUse = ControlledObjects ? *ControlledObjects : SelectedObjects;

	ColorGradingDataModel->SetObjects(ControlledObjectsToUse);

	FColorGradingPanelState PanelState;
	GetPanelState(PanelState);

	PanelState.SelectedObjects.Empty(SelectedObjects.Num());
	PanelState.SelectedObjects.Append(SelectedObjects);

	PanelState.ControlledObjects.Empty(ControlledObjectsToUse.Num());
	PanelState.ControlledObjects.Append(ControlledObjectsToUse);

	SetPanelState(PanelState);
}

TSharedRef<SWidget> SColorGradingPanel::CreateDockInLayoutButton()
{
	if (bIsInDrawer && DockCallback.IsBound())
	{
		return SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText(LOCTEXT("DockInLayout_Tooltip", "Docks this panel in the current window, copying all settings from the drawer.\nThe drawer will still be usable."))
			.OnClicked(this, &SColorGradingPanel::DockInLayout)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0, 0.0f)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("Icons.Layout"))
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(4.0, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DockInLayout", "Dock in Layout"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
	}

	return SNullWidget::NullWidget;
}

UWorld* SColorGradingPanel::GetWorld()
{
	if (OverrideWorld.IsSet() && OverrideWorld.Get())
	{
		return OverrideWorld.Get();
	}

	ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>();
	if (!LevelEditorSubsystem)
	{
		return nullptr;
	}

	ULevel* Level = LevelEditorSubsystem->GetCurrentLevel();
	if (!Level)
	{
		return nullptr;
	}

	return Level->GetWorld();
}

void SColorGradingPanel::RefreshColorGradingList()
{
	if (ObjectListModel)
	{
		ObjectListModel->RequestRebuildList();
	}
}

void SColorGradingPanel::FillColorGradingGroupToolBar()
{
	if (ColorGradingGroupToolBarBox.IsValid())
	{
		ColorGradingGroupToolBarBox->ClearChildren();
		ColorGradingGroupTextBlocks.Empty(ColorGradingDataModel->ColorGradingGroups.Num());

		for (int32 Index = 0; Index < ColorGradingDataModel->ColorGradingGroups.Num(); ++Index)
		{
			TSharedPtr<SInlineEditableTextBlock> TextBlock = nullptr;

			ColorGradingGroupToolBarBox->AddSlot()
			.AutoWidth()
			.Padding(2.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "DetailsView.SectionButton")
				.OnCheckStateChanged(this, &SColorGradingPanel::OnColorGradingGroupCheckedChanged, Index)
				.IsChecked(this, &SColorGradingPanel::IsColorGradingGroupSelected, Index)
				.OnGetMenuContent(this, &SColorGradingPanel::GetColorGradingGroupMenuContent, Index)
				[
					SNew(SBox)
					.HeightOverride(20.0)
					[
						SAssignNew(TextBlock, SInlineEditableTextBlock)
						.Text(this, &SColorGradingPanel::GetColorGradingGroupDisplayName, Index)
						.Font(this, &SColorGradingPanel::GetColorGradingGroupDisplayNameFont, Index)
						.OnTextCommitted(this, &SColorGradingPanel::OnColorGradingGroupRenamed, Index)
					]
				]
			];

			ColorGradingGroupTextBlocks.Add(TextBlock);
		}

		if (ColorGradingDataModel->ColorGradingGroupToolBarWidget.IsValid())
		{
			ColorGradingGroupToolBarBox->AddSlot()
			.AutoWidth()
			.Padding(2.0f, 0.0f, 2.0f, 0.0f)
			[
				ColorGradingDataModel->ColorGradingGroupToolBarWidget.ToSharedRef()
			];
		}
	}
}

EVisibility SColorGradingPanel::GetColorGradingGroupToolBarVisibility() const
{
	if (ColorGradingDataModel->bShowColorGradingGroupToolBar)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

ECheckBoxState SColorGradingPanel::IsColorGradingGroupSelected(int32 GroupIndex) const
{
	return ColorGradingDataModel->GetSelectedColorGradingGroupIndex() == GroupIndex ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SColorGradingPanel::OnColorGradingGroupCheckedChanged(ECheckBoxState State, int32 GroupIndex)
{
	if (State == ECheckBoxState::Checked)
	{
		ColorGradingDataModel->SetSelectedColorGradingGroup(GroupIndex);
	}
}

FText SColorGradingPanel::GetColorGradingGroupDisplayName(int32 GroupIndex) const
{
	if (ColorGradingDataModel->ColorGradingGroups.IsValidIndex(GroupIndex))
	{
		FText DisplayName = ColorGradingDataModel->ColorGradingGroups[GroupIndex].DisplayName;
		if (DisplayName.IsEmpty())
		{
			return LOCTEXT("ColorGradingGroupEmptyNameLabel", "Unnamed");
		}

		return DisplayName;
	}

	return FText::GetEmpty();
}

FSlateFontInfo SColorGradingPanel::GetColorGradingGroupDisplayNameFont(int32 GroupIndex) const
{
	if (ColorGradingDataModel->ColorGradingGroups.IsValidIndex(GroupIndex))
	{
		FText DisplayName = ColorGradingDataModel->ColorGradingGroups[GroupIndex].DisplayName;
		if (DisplayName.IsEmpty())
		{
			return FAppStyle::Get().GetFontStyle("NormalFontItalic");
		}
	}

	return FAppStyle::Get().GetFontStyle("NormalFont");
}

TSharedRef<SWidget> SColorGradingPanel::GetColorGradingGroupMenuContent(int32 GroupIndex)
{
	if (ColorGradingDataModel->ColorGradingGroups.IsValidIndex(GroupIndex))
	{
		const FColorGradingEditorDataModel::FColorGradingGroup& Group = ColorGradingDataModel->ColorGradingGroups[GroupIndex];

		FMenuBuilder MenuBuilder(true, nullptr);

		const FGenericCommands& GenericCommands = FGenericCommands::Get();

		if (Group.bCanBeRenamed)
		{
			MenuBuilder.AddMenuEntry(
				GenericCommands.Rename->GetLabel(),
				GenericCommands.Rename->GetDescription(),
				GenericCommands.Rename->GetIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SColorGradingPanel::OnColorGradingGroupRequestRename, GroupIndex))
			);
		}

		if (Group.bCanBeDeleted)
		{
			MenuBuilder.AddMenuEntry(
				GenericCommands.Delete->GetLabel(),
				GenericCommands.Delete->GetDescription(),
				GenericCommands.Delete->GetIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SColorGradingPanel::OnColorGradingGroupDeleted, GroupIndex))
			);
		}

		return MenuBuilder.MakeWidget();
	}

	return SNullWidget::NullWidget;
}

void SColorGradingPanel::OnColorGradingGroupDeleted(int32 GroupIndex)
{
	// If the group being deleted is in front of the currently selected one, we want to make sure
	// that the same group is selected even after the deletion, so preemptively adjust the
	// currently selected group index
	int32 SelectedGroupIndex = ColorGradingDataModel->GetSelectedColorGradingGroupIndex();
	if (SelectedGroupIndex >= GroupIndex)
	{
		ColorGradingDataModel->SetSelectedColorGradingGroup(SelectedGroupIndex - 1);
	}

	ColorGradingDataModel->OnColorGradingGroupDeleted().Broadcast(GroupIndex);
}

void SColorGradingPanel::OnColorGradingGroupRequestRename(int32 GroupIndex)
{
	if (ColorGradingGroupTextBlocks.IsValidIndex(GroupIndex) && ColorGradingGroupTextBlocks[GroupIndex].IsValid())
	{
		ColorGradingGroupTextBlocks[GroupIndex]->EnterEditingMode();
	}
}

void SColorGradingPanel::OnColorGradingGroupRenamed(const FText& InText, ETextCommit::Type TextCommitType, int32 GroupIndex)
{
	ColorGradingDataModel->OnColorGradingGroupRenamed().Broadcast(GroupIndex, InText);
}

void SColorGradingPanel::OnColorGradingDataModelGenerated()
{
	FillColorGradingGroupToolBar();

	if (ColorWheelPanel.IsValid())
	{
		ColorWheelPanel->Refresh();
	}
}

void SColorGradingPanel::OnListSelectionChanged(FSceneOutlinerTreeItemPtr TreeItem, ESelectInfo::Type Type)
{
	if (Type == ESelectInfo::Direct)
	{
		return;
	}

	UpdateSelectionFromList();
}

void SColorGradingPanel::OnListSelectionSynchronized()
{
	UpdateSelectionFromList();
}

void SColorGradingPanel::UpdateSelectionFromList()
{
	TArray<FSceneOutlinerTreeItemPtr> SelectedOutlinerItems = ColorGradingObjectListView->GetSelectedItems();
	TArray<UObject*> SelectedObjects;
	TArray<UObject*> ControlledObjects;

	GetSelectedAndControlledObjects(SelectedOutlinerItems, SelectedObjects, ControlledObjects);

	TArray<TWeakObjectPtr<UObject>> OldControlledObjects = ColorGradingDataModel->GetObjects();
	if (!Algo::Compare(OldControlledObjects, ControlledObjects))
	{
		SetSelectedObjects(SelectedObjects, &ControlledObjects);
	}
}

void SColorGradingPanel::GetSelectedAndControlledObjects(const TArray<FSceneOutlinerTreeItemPtr>& InSelectedItems, TArray<UObject*>& OutSelectedObjects, TArray<UObject*>& OutControlledObjects) const
{
	OutSelectedObjects.Reserve(InSelectedItems.Num());
	OutControlledObjects.Reserve(InSelectedItems.Num());

	for (FSceneOutlinerTreeItemPtr TreeItem : InSelectedItems)
	{
		if (!TreeItem.IsValid())
		{
			continue;
		}

		if (const FActorTreeItem* ActorTreeItem = TreeItem->CastTo<FActorTreeItem>())
		{
			if (AActor* Actor = ActorTreeItem->Actor.Get())
			{
				// Check if this actor is listed as an associated actor, in which case its parent is the actual selection target for color grading
				if (const FObjectMixerEditorListRowActor* ObjectMixerActorItem = TreeItem->CastTo<FObjectMixerEditorListRowActor>())
				{
					if (AActor* ParentActor = ObjectMixerActorItem->OverrideParent.Get())
					{
						if (const IColorGradingMixerObjectHierarchyConfig* HierarchyConfig = FColorGradingMixerObjectFilterRegistry::GetClassHierarchyConfig(ParentActor->GetClass()))
						{
							if (HierarchyConfig->IsActorAssociated(ParentActor, Actor))
							{
								OutControlledObjects.Add(ParentActor);
								OutSelectedObjects.Add(Actor);
								continue;
							}
						}
					}
				}

				// Otherwise we want to control it directly
				OutControlledObjects.Add(Actor);
				OutSelectedObjects.Add(Actor);
				continue;
			}
		}

		if (const FComponentTreeItem* ComponentTreeItem = TreeItem->CastTo<FComponentTreeItem>())
		{
			if (UActorComponent* Component = ComponentTreeItem->Component.Get())
			{
				OutControlledObjects.Add(Component);
				OutSelectedObjects.Add(Component);
				continue;
			}
		}

		if (const FObjectMixerEditorListRowUObject* ObjectTreeItem = TreeItem->CastTo<FObjectMixerEditorListRowUObject>())
		{
			if (UObject* Object = ObjectTreeItem->ObjectSoftPtr.Get())
			{
				OutControlledObjects.Add(Object);
				OutSelectedObjects.Add(Object);
				continue;
			}
		}
	}
}

FReply SColorGradingPanel::DockInLayout()
{
	DockCallback.ExecuteIfBound();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
