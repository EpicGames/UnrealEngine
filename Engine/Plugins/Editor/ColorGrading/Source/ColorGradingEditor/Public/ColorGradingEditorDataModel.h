// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailPropertyRow.h"
#include "PropertyEditorDelegates.h"

#define UE_API COLORGRADINGEDITOR_API

class FResetToDefaultOverride;
class IPropertyHandle;
class IPropertyRowGenerator;
class SWidget;
struct FColorGradingPanelState;

/** Interface that allows color grading data models to be generated for specific UObjects */
class IColorGradingEditorDataModelGenerator : public TSharedFromThis<IColorGradingEditorDataModelGenerator>
{
public:
	virtual ~IColorGradingEditorDataModelGenerator() = default;

	virtual void Initialize(const TSharedRef<class FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator) = 0;
	virtual void Destroy(const TSharedRef<class FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator) = 0;

	/** Called when the color grading data model should be generated by this generator */
	virtual void GenerateDataModel(IPropertyRowGenerator& PropertyRowGenerator, class FColorGradingEditorDataModel& OutColorGradingDataModel) = 0;
};

DECLARE_DELEGATE_RetVal(TSharedRef<IColorGradingEditorDataModelGenerator>, FGetDetailsDataModelGenerator);

/**
 * A data model that stores the color grading properties from a UObject, which allows UObjects with widely varying color grading
 * interfaces to be used with the color grading panel.
 */
class FColorGradingEditorDataModel : public TSharedFromThis<FColorGradingEditorDataModel>
{
public:
	/** Registers a new data model generator used to populate a color grading data model for the specified class */
	template<class T>
	static void RegisterColorGradingDataModelGenerator(FGetDetailsDataModelGenerator CreateGeneratorDelegate)
	{
		UClass* Class = T::StaticClass();
		RegisteredDataModelGenerators.Add(Class, CreateGeneratorDelegate);
	}

private:
	/** A list of data model generators, indexed by class name, that have been registered */
	static UE_API TMap<TWeakObjectPtr<UClass>, FGetDetailsDataModelGenerator> RegisteredDataModelGenerators;

public:
	/** Stores the property handles for the color properties for a single color grading element (Shadows, Midtones, etc.) */
	struct FColorGradingElement
	{
		/** The display name of the color grading element */
		FText DisplayName;

		/** Property handle of the color property used for saturation */
		TSharedPtr<IPropertyHandle> SaturationPropertyHandle;

		/** Optional reset to default handler for the saturation property */
		TOptional<FResetToDefaultOverride> SaturationResetToDefaultOverride;
		
		/** Property handle of the color property used for contrast */
		TSharedPtr<IPropertyHandle> ContrastPropertyHandle;

		/** Optional reset to default handler for the contrast property */
		TOptional<FResetToDefaultOverride> ContrastResetToDefaultOverride;
		
		/** Property handle of the color property used for gamma */
		TSharedPtr<IPropertyHandle> GammaPropertyHandle;

		/** Optional reset to default handler for the gamma property */
		TOptional<FResetToDefaultOverride> GammaResetToDefaultOverride;
		
		/** Property handle of the color property used for gain */
		TSharedPtr<IPropertyHandle> GainPropertyHandle;

		/** Optional reset to default handler for the gain property */
		TOptional<FResetToDefaultOverride> GainResetToDefaultOverride;
		
		/** Property handle of the color property used for offset */
		TSharedPtr<IPropertyHandle> OffsetPropertyHandle;

		/** Optional reset to default handler for the offset property */
		TOptional<FResetToDefaultOverride> OffsetResetToDefaultOverride;
	};

	/** Stores a list of color grading elements that form a single color grading group */
	struct FColorGradingGroup
	{
		/** The display name of the color grading group */
		FText DisplayName = FText::GetEmpty();
		
		/** The handle of the property that the color grading properties are generated from */
		TSharedPtr<IPropertyHandle> GroupPropertyHandle = nullptr;

		/** The widget displayed in the color wheel panel's header for this group, usually just a text block to show the group's name */
		TSharedPtr<SWidget> GroupHeaderWidget = nullptr;

		/** The handle of the property that controls the editing condition of the properties in the color grading group */
		TSharedPtr<IPropertyHandle> EditConditionPropertyHandle = nullptr;

		/** The list of color grading elements (Shadows, Midtones, etc.) that belong in the group */
		TArray<FColorGradingElement> ColorGradingElements;

		/** A list of categories of properties to display in the detail view of the color grading panel */
		TArray<FName> DetailsViewCategories;

		/** Indicates that the color grading group can be deleted by the user */
		bool bCanBeDeleted = false;

		/** Indicates that the color grading group can be renamed by the user */
		bool bCanBeRenamed = false;
	};

	DECLARE_MULTICAST_DELEGATE(FOnDataModelGenerated);
	DECLARE_MULTICAST_DELEGATE(FOnColorGradingSelectionChanged);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnColorGradingGroupDeleted, int32);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnColorGradingGroupRenamed, int32, const FText&);

public:
	UE_API FColorGradingEditorDataModel();

	/** Gets the property row generator that was used to generate this data model */
	TSharedRef<IPropertyRowGenerator> GetPropertyRowGenerator() const { return PropertyRowGenerator.ToSharedRef(); }

	/** Gets the list of objects from which the color grading properties were generated from */
	UE_API TArray<TWeakObjectPtr<UObject>> GetObjects() const;

	/** Sets the objects from which the color grading properties will be generated from */
	UE_API void SetObjects(const TArray<UObject*>& InObjects);

	/** Gets whether the data model has objects of the specified class */
	UE_API bool HasObjectOfType(const UClass* InClass) const;

	/** Resets the data model to be empty */
	UE_API void Reset();

	/** Adds the state of the data model to the specified drawer state */
	UE_API void GetPanelState(FColorGradingPanelState& OutPanelState);

	/** Sets the state of the data model from the specified drawer state */
	UE_API void SetPanelState(const FColorGradingPanelState& InPanelState);

	/** Gets the index of the selected color grading group */
	int32 GetSelectedColorGradingGroupIndex() const { return SelectedColorGradingGroupIndex; }

	/** Gets a pointer to the selected color grading group, or null if there isn't a valid selection */
	UE_API FColorGradingGroup* GetSelectedColorGradingGroup();

	/** Sets the selected color grading group */
	UE_API void SetSelectedColorGradingGroup(int32 InColorGradingGroupIndex);

	/** Gets the delegate that is raised when the data model is generated */
	FOnDataModelGenerated& OnDataModelGenerated() { return OnDataModelGeneratedDelegate; }

	/** Gets the delegate that is raised when the selected color grading group has changed */
	FOnColorGradingSelectionChanged& OnColorGradingGroupSelectionChanged() { return OnColorGradingGroupSelectionChangedDelegate; }

	/** Gets the index of the selected color grading element */
	int32 GetSelectedColorGradingElementIndex() const { return SelectedColorGradingElementIndex; }

	/** Gets a pointer to the selected color grading element, or null if there isn't a valid selection */
	UE_API FColorGradingElement* GetSelectedColorGradingElement();

	/** Sets the selected color grading element */
	UE_API void SetSelectedColorGradingElement(int32 InColorGradingElementIndex);

	/** Gets the delegate that is raised when the selected color grading element has changed */
	FOnColorGradingSelectionChanged& OnColorGradingElementSelectionChanged() { return OnColorGradingElementSelectionChangedDelegate; }

	/** Gets the delegate that is raised when a color grading group is deleted by the user */
	FOnColorGradingGroupDeleted& OnColorGradingGroupDeleted() { return OnColorGradingGroupDeletedDelegate; }

	/** Gets the delegate that is raised when a color grading group is renamed by the user */
	FOnColorGradingGroupRenamed& OnColorGradingGroupRenamed() { return OnColorGradingGroupRenamedDelegate; }

private:
	/** Initializes any needed data model generators for the specified class */
	UE_API void InitializeDataModelGenerator(UClass* InClass);

	/** Attempts to find an instance of a data model generator that can be used for the specified class */
	UE_API TSharedPtr<IColorGradingEditorDataModelGenerator> GetDataModelGenerator(UClass* InClass) const;

	/** Callback that is raised when the property row generator the data model is generated from has been refreshed */
	UE_API void OnPropertyRowGeneratorRefreshed();

public:
	/** A list of color grading groups in the color grading data model */
	TArray<FColorGradingGroup> ColorGradingGroups;

	/** An optional widget to append to the end of the color grading group toolbar */
	TSharedPtr<SWidget> ColorGradingGroupToolBarWidget = nullptr;

	/** Indicates whether the color grading groups should be visible/selectable in a toolbar */
	bool bShowColorGradingGroupToolBar = false;

private:
	/** A list of data model generators that have been created to generate the data model for the current objects */
	TMap<TWeakObjectPtr<UClass>, TSharedPtr<IColorGradingEditorDataModelGenerator>> DataModelGeneratorInstances;

	/** The property row generator which manages the property handles of the color grading objects */
	TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator;

	/** The currently selected color grading element */
	int32 SelectedColorGradingGroupIndex = INDEX_NONE;

	/** The currently selected color grading element */
	int32 SelectedColorGradingElementIndex = INDEX_NONE;

	/** Delegate that is raised when the data model has been generated */
	FOnDataModelGenerated OnDataModelGeneratedDelegate;

	/** Delegate that is raised when the selected color grading group has been changed */
	FOnColorGradingSelectionChanged OnColorGradingGroupSelectionChangedDelegate;

	/** Delegate that is raised when the selected color grading element has been changed */
	FOnColorGradingSelectionChanged OnColorGradingElementSelectionChangedDelegate;

	/** Delegate that is raised when a color grading group is being deleted */
	FOnColorGradingGroupDeleted OnColorGradingGroupDeletedDelegate;

	/** Delegate that is raised when a color grading group is being renamed */
	FOnColorGradingGroupRenamed OnColorGradingGroupRenamedDelegate;
};

#undef UE_API
