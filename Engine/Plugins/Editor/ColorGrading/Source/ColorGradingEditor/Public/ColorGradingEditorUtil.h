// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/SWidget.h"

namespace ColorGradingEditorUtil
{

/**
 * Make a button that opens the Color Grading editor.
 * @param bWrapInBox If true, wrap the button in a box that centers it and adds standard padding for use in the Details panel.
 */
COLORGRADINGEDITOR_API TSharedRef<SWidget> MakeColorGradingLaunchButton(bool bWrapInBox = true);

}
