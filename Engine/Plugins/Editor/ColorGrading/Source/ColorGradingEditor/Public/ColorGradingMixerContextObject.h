// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ColorGradingMixerContextObject.generated.h"	

class UToolMenu;

/** Empty class which can be added to an FToolMenuContext to indicate that the menu originates from the Color Grading panel. */
UCLASS(MinimalAPI)
class UColorGradingMixerContextObject : public UObject
{
	GENERATED_BODY()
};
