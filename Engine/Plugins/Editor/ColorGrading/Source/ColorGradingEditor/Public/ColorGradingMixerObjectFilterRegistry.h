// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "SceneOutlinerDragDrop.h"
#include "ObjectFilter/ObjectMixerEditorActorSubObject.h"
#include "Templates/SubclassOf.h"

#define UE_API COLORGRADINGEDITOR_API

class UClass;

/** Interface to configure how an object is displayed in the Color Grading panel's outliner */
struct IColorGradingMixerObjectHierarchyConfig : public TSharedFromThis<IColorGradingMixerObjectHierarchyConfig>
{
public:
	virtual ~IColorGradingMixerObjectHierarchyConfig() = default;

	/** Generate a list of additional actors to include as children in the ObjectMixer hierarchy for the given object */
	virtual TArray<AActor*> FindAssociatedActors(UObject* ParentObject) const { return {}; }

	/**
	 * Return true if AssociatedActor is associated with ParentObject.
	 * This should return true if the actor would be included in FindAssociatedActors, but is a separate function to avoid allocating
	 * an array every time.
	 */
	virtual bool IsActorAssociated(UObject* ParentObject, AActor* AssociatedActor) const { return false; }

	/** Generate a list of additional sub-objects to include as children in the ObjectMixer hierarchy for the given object */
	virtual TArray<FObjectMixerEditorActorSubObject> GetActorSubObjects(UObject* ParentObject) const { return {}; }

	/** 
	 * If true, this has custom handling for drop operations and will override the hierarchy's default behaviour with the ValidateDrop
	 * and OnDrop functions.
	 */
	virtual bool HasCustomDropHandling() const { return false; }

	/** Test whether the specified payload can be dropped onto a tree item representing this object */
	virtual FSceneOutlinerDragValidationInfo ValidateDrop(UObject* DropTarget, const FSceneOutlinerDragDropPayload& Payload) const { return FSceneOutlinerDragValidationInfo::Invalid(); }

	/** Called when a payload is dropped onto a target. If this returns true, the outliner will be refreshed after the operation. */
	virtual bool OnDrop(UObject* DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const { return false; }

	/**
	 * If a property is changed that has a name found in this set, the color grading mixer hierarchy will be refreshed.
	 * Add a property name to this list if you expect the list to change in some way after changing that property.
	 */
	virtual TSet<FName> GetPropertiesThatRequireListRefresh() const { return {}; }
};

DECLARE_DELEGATE_RetVal(TSharedRef<IColorGradingMixerObjectHierarchyConfig>, FGetObjectHierarchyConfig);

/** Interface for custom selection handlers that can receive selection events from the color grading drawer's mixer */
struct IColorGradingMixerSelectionHandler : public TSharedFromThis<IColorGradingMixerSelectionHandler>
{
public:
	DECLARE_DELEGATE(FOnSelectionChanged);

	virtual ~IColorGradingMixerSelectionHandler() = default;
	
	/** Returns true if this handler can handle synchronizing the specified object */
	virtual bool CanSelectSubObject(const UObject* InObject) { return false; }
	
	/** Called when the color grading mixer synchronizes its selection to the rest of the editor */
	virtual void SelectSubObjects(const TArray<UObject*>& InSelectedObjects, bool bShouldSelect, bool bSelectEvenIfHidden) const { }

	/** Called when the color grading mixer synchronizes its selection from selected objects in the editor. Retrieves a list of all sub-objects selected in the editor */
	virtual TArray<UObject*> GetSelectedSubObjectsInEditor() const { return {}; }

	/** Delegate that can be invoked to cause the color grading mixer to re-synchronize its selection to the editor's selection */
	FOnSelectionChanged OnSelectionChanged;
};

DECLARE_DELEGATE_RetVal(TSharedRef<IColorGradingMixerSelectionHandler>, FGetSelectionHandler);

/**
 * Contains functions for controlling which classes can be managed from the Color Grading panel's ObjectMixer-based hierarchy panel
 */
struct FColorGradingMixerObjectFilterRegistry
{
public:

	/**
	 * Register an object class that can be seen in a Color Grading panel's object list.
	 * @param Class The object class to register
	 * @param Config Optional delegate that creates a configuration for how the object will be displayed in the hierarchy
	 */
	static void RegisterObjectClassToFilter(UClass* Class, FGetObjectHierarchyConfig CreateConfigDelegate = nullptr)
	{
		ObjectClassesToFilter.Add(Class);

		if (CreateConfigDelegate.IsBound())
		{
			HierarchyConfigs.Add(Class, CreateConfigDelegate);
		}
	}

	/** Register an actor class that can be placed from the Color Grading panel's object list */
	static void RegisterActorClassToPlace(TSubclassOf<AActor> Class)
	{
		ActorClassesToPlace.Add(Class);
	}

	/**
	 * Registers a new selection handler with the color drawer that will receive selection related events from the drawer's mixer
	 * @param InCreateHandlerDelegate A delegate that is invoked to create the selection handler when needed
	 */
	static void RegisterSelectionHandler(FGetSelectionHandler InCreateHandlerDelegate)
	{
		SelectionHandlers.Add(InCreateHandlerDelegate);	
	}
	
	/** Get the set of object classes that can be seen in a Color Grading panel's object list */
	static const TSet<UClass*>& GetObjectClassesToFilter() { return ObjectClassesToFilter; }

	/** Get the set of actor classes that can be placed from a Color Grading panel's object list */
	static const TSet<TSubclassOf<AActor>>& GetActorClassesToPlace() { return ActorClassesToPlace; }

	/** Get the hierarchy configuration for a class, or null if none was provided */
	static UE_API const IColorGradingMixerObjectHierarchyConfig* GetClassHierarchyConfig(TSubclassOf<AActor> Class);

	/** Instantiates a list of all selection handlers that have been registered to the color grading drawer */
	static TArray<TSharedPtr<IColorGradingMixerSelectionHandler>> CreateSelectionHandlers();
	
private:
	/** Set of classes that can be seen in the object panel */
	UE_API inline static TSet<UClass*> ObjectClassesToFilter;

	/** Set of classes that can be placed from the object panel */
	UE_API inline static TSet<TSubclassOf<AActor>> ActorClassesToPlace;

	/** Map from object class to hierarchy configuration, if provided */
	UE_API inline static TMap<TWeakObjectPtr<UClass>, FGetObjectHierarchyConfig> HierarchyConfigs;

	/** Map from object class to hierarchy configuration instance. If null, none exist for the class */
	UE_API inline static TMap<TWeakObjectPtr<UClass>, TSharedPtr<IColorGradingMixerObjectHierarchyConfig>> HierarchyConfigInstances;

	/** List of handlers that process custom selection synchronization for specific color gradable objects */
	UE_API inline static TArray<FGetSelectionHandler> SelectionHandlers;
};

#undef UE_API
