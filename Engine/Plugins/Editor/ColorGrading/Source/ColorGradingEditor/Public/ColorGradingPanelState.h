// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateTypes.h"
#include "Framework/ColorGrading/ColorGradingCommon.h"

/**
 * Stores the state of the color grading panel UI that can be reloaded in cases where the panel or any of its elements
 * are reloaded (such as when the containing drawer is reopened or docked)
 */
struct FColorGradingPanelState
{
	/** The objects that are selected in the list */
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;

	/**
	 * The objects that are being controlled by the color grading controls.
	 * This can differ from SelectedObjects when SelectedObjects contains objects that are associated
	 * with a color-gradable object, but don't have controls themselves.
	 */
	TArray<TWeakObjectPtr<UObject>> ControlledObjects;

	/** The color grading group that is selected */
	int32 SelectedColorGradingGroup = INDEX_NONE;

	/** The color grading element that is selected */
	int32 SelectedColorGradingElement = INDEX_NONE;

	/** The color display mode of the color wheels */
	UE::ColorGrading::EColorGradingColorDisplayMode ColorDisplayMode;

	/** Indicates which subsections were selected for each section in the details panel */
	TArray<int32> SelectedDetailsSubsections;
};
