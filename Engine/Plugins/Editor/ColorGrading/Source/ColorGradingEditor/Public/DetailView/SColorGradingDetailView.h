// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DetailColumnSizeData.h"
#include "DetailWidgetRow.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"

#define UE_API COLORGRADINGEDITOR_API

class IDetailTreeNode;
class IPropertyRowGenerator;
class IPropertyHandle;
class ITableRow;
class FColorGradingDetailTreeItem;
class FDetailTreeNode;
class FDetailWidgetRow;
class STableViewBase;

template<typename T>
class STreeView;

DECLARE_DELEGATE_RetVal_OneParam(bool, FOnFilterDetailTreeNode, const TSharedRef<IDetailTreeNode>&);

/**
 * A custom detail view based on SDetailView that uses a property row generator as a source for the property nodes instead of generating them manually.
 * Using an existing property row generator allows the detail view to display an object's properties much faster than the ordinary SDetailView, which
 * has to regenerate a new property node tree every time the object being displayed is changed
 */
class SColorGradingDetailView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SColorGradingDetailView) { }
		SLATE_ARGUMENT(TSharedPtr<IPropertyRowGenerator>, PropertyRowGeneratorSource)
		SLATE_EVENT(FOnFilterDetailTreeNode, OnFilterDetailTreeNode)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	//~ SWidget interface
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget interface

	/** Regenerates this widget based on the current state of its property row generator source */
	UE_API void Refresh();

	/** Saves the expansion state of all properties being displayed in this detail view to the user's config file */
	UE_API void SaveExpandedItems();

	/** Restores the expansion state of all properties being displayed in this detail view from the user's config file */
	UE_API void RestoreExpandedItems();

private:
	/** Updates the detail tree using the current state of the property row generator source */
	UE_API void UpdateTreeNodes();

	/** Updates the expansion state of the specified tree item using the stored expansion state configuration */
	UE_API void UpdateExpansionState(const TSharedRef<FColorGradingDetailTreeItem> InTreeItem);

	/** Sets the expansion state of the specified tree item, and optionally recursively sets the expansion state of its children */
	UE_API void SetNodeExpansionState(TSharedRef<FColorGradingDetailTreeItem> InTreeNode, bool bIsItemExpanded, bool bRecursive);

	/** Generates a table row widget for the specified tree item */
	UE_API TSharedRef<ITableRow> GenerateNodeRow(TSharedRef<FColorGradingDetailTreeItem> InTreeNode, const TSharedRef<STableViewBase>& OwnerTable);

	/** Gets a list of child tree items for the specified tree item */
	UE_API void GetChildrenForNode(TSharedRef<FColorGradingDetailTreeItem> InTreeNode, TArray<TSharedRef<FColorGradingDetailTreeItem>>& OutChildren);

	/** Raised when the underlying tree widget is setting the expansion state of the specified tree item recursively */
	UE_API void OnSetExpansionRecursive(TSharedRef<FColorGradingDetailTreeItem> InTreeNode, bool bIsItemExpanded);

	/** Raised when the underlying tree widget is setting the expansion state of the specified tree item */
	UE_API void OnExpansionChanged(TSharedRef<FColorGradingDetailTreeItem> InTreeNode, bool bIsItemExpanded);

	/** Raised when the underlying tree widget is releasing the specified table row */
	UE_API void OnRowReleased(const TSharedRef<ITableRow>& TableRow);

	/** Gets the visibility of the scrollbar for the detail view */
	UE_API EVisibility GetScrollBarVisibility() const;

private:
	typedef STreeView<TSharedRef<FColorGradingDetailTreeItem>> SDetailTree;

	/** The underlying tree view used to display the property widgets */
	TSharedPtr<SDetailTree> DetailTree;

	/** The source list of the root detail tree nodes being displayed by the tree widget */
	TArray<TSharedRef<FColorGradingDetailTreeItem>> RootTreeNodes;

	/** The property row generator to generate the property widgets from */
	TSharedPtr<IPropertyRowGenerator> PropertyRowGeneratorSource;

	/** Column sizing data for the properties */
	FDetailColumnSizeData ColumnSizeData;

	/** A list of tree items whose expansion state needs to be set on the next tick */
	TMap<TWeakPtr<FColorGradingDetailTreeItem>, bool> TreeItemsToSetExpansionState;

	/** A list of currently expanded detail nodes */
	TSet<FString> ExpandedDetailNodes;

	/** Delegate used to filter or process the detail tree nodes that are displayed in the detail view */
	FOnFilterDetailTreeNode OnFilterDetailTreeNode;
};

#undef UE_API
