// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "EditorUndoClient.h"
#include "SceneOutlinerFwd.h"
#include "SelectionInterface/IObjectMixerSelectionInterface.h"

#include "ColorGradingPanelState.h"

#define UE_API COLORGRADINGEDITOR_API

class FColorGradingEditorDataModel;
class FObjectMixerEditorList;
class SColorGradingColorWheelPanel;
class SHorizontalBox;
class SInlineEditableTextBlock;
class UWorld;

using FColorGradingActorFilter = TFunction<bool(const AActor*)>;

/** Main panel of a color grading drawer widget, which displays color wheels or selected object details */
class SColorGradingPanel : public SCompoundWidget, public FEditorUndoClient
{
public:
	UE_API ~SColorGradingPanel();

	SLATE_BEGIN_ARGS(SColorGradingPanel)
		: _IsInDrawer(false)
		{}

		/** Indicates whether this widget is in a drawer or docked in a tab */
		SLATE_ARGUMENT(bool, IsInDrawer)

		/** The world in which to search for actors to display for editing. If not provided, the level editor's current world will be used */
		SLATE_ATTRIBUTE(UWorld*, OverrideWorld)

		/** Event invoked when the user presses the dock button */
		SLATE_EVENT(FSimpleDelegate, OnDocked)

		/** Function which, if it returns false when passed an actor, filters it and its sub-entries out of the color grading item list */
		SLATE_ARGUMENT(FColorGradingActorFilter, ActorFilter)

		/** Optional interface which, if provided, will determine how objects selected in this panel will be synchronized with the rest of the editor */
		SLATE_ARGUMENT(TSharedPtr<IObjectMixerSelectionInterface>, SelectionInterface)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	/** Refreshes the panel's UI to match the current state of the level */
	UE_API void Refresh();

	/** Gets the state of the panel UI */
	UE_API void GetPanelState(FColorGradingPanelState& OutPanelState) const;

	/** Sets the state of the panel UI */
	UE_API void SetPanelState(const FColorGradingPanelState& InPanelState);

	/**
	 * Set the list of selected objects, updating state and data model as appropriate.
	 * If ControlledObjects is provided, only the objects in that array will be used to display the color grading wheels.
	 * Otherwise, the controlled objects will use the contents of SelectedObjects.
	 */
	UE_API void SetSelectedObjects(const TArray<UObject*>& SelectedObjects, const TArray<UObject*>* ControlledObjects = nullptr);

private:
	/** Creates the button used to dock the drawer in the operator panel */
	UE_API TSharedRef<SWidget> CreateDockInLayoutButton();

	/** Get the world currently being edited */
	UE_API UWorld* GetWorld();

	/** Refreshes the object list, filling it with the current color gradable objects from the root actor and world */
	UE_API void RefreshColorGradingList();

	/** Fills the color grading group toolbar using the color grading data model */
	UE_API void FillColorGradingGroupToolBar();

	/** Gets the visibility state of the color grading group toolbar */
	UE_API EVisibility GetColorGradingGroupToolBarVisibility() const;

	/** Gets whether the color grading group at the specified index is currently selected */
	UE_API ECheckBoxState IsColorGradingGroupSelected(int32 GroupIndex) const;

	/** Raised when the user has selected the specified color grading group */
	UE_API void OnColorGradingGroupCheckedChanged(ECheckBoxState State, int32 GroupIndex);

	/** Gets the display name of the specified color grading group */
	UE_API FText GetColorGradingGroupDisplayName(int32 GroupIndex) const;

	/** Gets the font of the display name label of the specified color grading group */
	UE_API FSlateFontInfo GetColorGradingGroupDisplayNameFont(int32 GroupIndex) const;

	/** Gets the content for the right click menu for the color grading group */
	UE_API TSharedRef<SWidget> GetColorGradingGroupMenuContent(int32 GroupIndex);

	/** Raised when a color grading group has been deleted by the user */
	UE_API void OnColorGradingGroupDeleted(int32 GroupIndex);

	/** Raised when a rename has been requested on a color grading group */
	UE_API void OnColorGradingGroupRequestRename(int32 GroupIndex);

	/** Raised when a rename has been committed on a color grading group */
	UE_API void OnColorGradingGroupRenamed(const FText& InText, ETextCommit::Type TextCommitType, int32 GroupIndex);

	/** Raised when the color grading data model has been generated */
	UE_API void OnColorGradingDataModelGenerated();

	/** Raised when the "Dock in Layout" button has been clicked */
	UE_API FReply DockInLayout();

	/** Raised when the user has selected a new item in any of the drawer's list views */
	UE_API void OnListSelectionChanged(FSceneOutlinerTreeItemPtr TreeItem, ESelectInfo::Type Type);

	/** Raised when the drawer's outliner has been synchronized with the editor selection */
	UE_API void OnListSelectionSynchronized();

	/** Update the selected items to match the list view */
	UE_API void UpdateSelectionFromList();

	/** Given an array of selected tree items, determine which objects will be selected and/or controlled */
	UE_API void GetSelectedAndControlledObjects(const TArray<FSceneOutlinerTreeItemPtr>& InSelectedItems, TArray<UObject*>& OutSelectedObjects, TArray<UObject*>& OutControlledObjects) const;

private:
	/** Model for the object mixer list used to display the color gradable object hierarchy. */
	TSharedPtr<FObjectMixerEditorList> ObjectListModel;

	/** Box containing the color grading groups */
	TSharedPtr<SHorizontalBox> ColorGradingGroupToolBarBox;

	/** List of editable text blocks containing color grading group names */
	TArray<TSharedPtr<SInlineEditableTextBlock>> ColorGradingGroupTextBlocks;

	/** Panel containing the color wheels */
	TSharedPtr<SColorGradingColorWheelPanel> ColorWheelPanel;

	/** The world from which to retrieve actors, if one was provided */
	TAttribute<UWorld*> OverrideWorld;

	/** Color grading object list widget being displayed in the drawer's list panel */
	TSharedPtr<SSceneOutliner> ColorGradingObjectListView;

	/** The color grading data model for the currently selected objects */
	TSharedPtr<FColorGradingEditorDataModel> ColorGradingDataModel;

	/** Indicates whether this widget is in a drawer or docked in a tab */
	bool bIsInDrawer;

	/** Indicates that the panel should refresh itself on the next tick */
	bool bRefreshOnNextTick = false;

	/** The function to call when the user presses the dock button */
	FSimpleDelegate DockCallback;

	/** Function used to filter actors before adding them to the object list. */
	FColorGradingActorFilter ActorFilter;
};

#undef UE_API
